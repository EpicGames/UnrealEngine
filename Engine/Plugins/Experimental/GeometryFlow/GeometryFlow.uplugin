{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "GeometryFlow",
	"Description": "Geometry DataFlow Graph",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "GeometryFlowCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GeometryFlowMeshProcessing",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},		
		{
			"Name": "GeometryFlowMeshProcessingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true
		}
	]
}
