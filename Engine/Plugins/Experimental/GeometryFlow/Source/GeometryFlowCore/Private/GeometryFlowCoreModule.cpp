// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFlowCoreModule.h"
#include "Modules/ModuleManager.h"

#include "GeometryFlowCoreNodeRegistration.h"

#define LOCTEXT_NAMESPACE "FGeometryFlowCoreModule"

void FGeometryFlowCoreModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

		// Register the nodes defined by this module with the GeometryFlowNodeFactory.
	UE::GeometryFlow::FCoreNodeRegistration::RegisterNodes();
}

void FGeometryFlowCoreModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryFlowCoreModule, GeometryFlowCore)
