// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFlowGraph.h"
#include "Async/Async.h"
#include "Algo/Find.h"
#include "UObject/UE5MainStreamObjectVersion.h"

using namespace UE::GeometryFlow;

DEFINE_LOG_CATEGORY(LogGeometryFlowGraph);

namespace
{
	// probably should be something defined for the whole tool framework. UETOOL-2989.
	static EAsyncExecution GeometryFlowGraphAsyncExecTarget = EAsyncExecution::Thread;

	// Can't seem to use threadpool in GeometryProcessingUnitTests. GThreadPool is null.
}

FGraph::~FGraph()
{
	EvaluateLock.Lock();    // Wait for any EvaluateResult to finish and also prevent any new evals from happening while destructing
}

TSafeSharedPtr<FNode> FGraph::FindNode(FHandle NodeHandle) const
{
	const FNodeInfo* Found = AllNodes.Find(NodeHandle);
	if (!ensure(Found != nullptr))
	{
		return nullptr;
	}
	return Found->Node;
}

TSafeSharedPtr<FRWLock> FGraph::FindNodeLock(FHandle NodeHandle) const
{
	const TSafeSharedPtr<FRWLock>* Found = AllNodeLocks.Find(NodeHandle);
	if (!ensure(Found != nullptr))
	{
		return nullptr;
	}
	return *Found;
}

EGeometryFlowResult FGraph::GetInputTypeForNode(FHandle NodeHandle, FString InputName, int32& Type) const
{
	const FNodeInfo* Found = AllNodes.Find(NodeHandle);
	if (!ensure(Found != nullptr))
	{
		return EGeometryFlowResult::NodeDoesNotExist;
	}
	EGeometryFlowResult Result = Found->Node->GetInputType(InputName, Type);
	return Result;
}

EGeometryFlowResult FGraph::GetOutputTypeForNode(FHandle NodeHandle, FString OutputName, int32& Type) const
{
	const FNodeInfo* Found = AllNodes.Find(NodeHandle);
	if (!ensure(Found != nullptr))
	{
		return EGeometryFlowResult::NodeDoesNotExist;
	}
	EGeometryFlowResult Result = Found->Node->GetOutputType(OutputName, Type);
	return Result;
}


ENodeCachingStrategy FGraph::GetCachingStrategyForNode(FHandle NodeHandle) const
{
	const FNodeInfo* Found = AllNodes.Find(NodeHandle);
	if (!ensure(Found != nullptr))
	{
		return ENodeCachingStrategy::AlwaysCache;
	}
	return (Found->CachingStrategy == ENodeCachingStrategy::Default) ?
		DefaultCachingStrategy : Found->CachingStrategy;
}

FGraph::FHandle FGraph::AddNodeOfType(
	const FName TypeName,
	const FString& Identifier,
	ENodeCachingStrategy CachingStrategy)
{
	FHandle Handle;
	Handle.Identifier = NodeCounter + 1;

	if (AddNodeOfType(TypeName, Handle, Identifier, CachingStrategy) == ENodeAddResult::Success)
	{
		NodeCounter++;
	}
	else
	{
		Handle = FHandle(); // invalid handle.
		UE_LOGF(LogGeometryFlowGraph, Warning, "Unregisterd type '%ls' : Failed to add node '%ls' ", *TypeName.ToString() , *Identifier);
	}

	return Handle;
}

bool  FGraph::IsDependent(FHandle DependentNode, FHandle IndependentNode) const
{
	const FNodeInfo* DependentNodeInfo = AllNodes.Find(DependentNode);
	const FNodeInfo* IndependentNodeInfo = AllNodes.Find(IndependentNode);

	if (!DependentNodeInfo || !IndependentNodeInfo) return false;

	// visit all nodes up-stream from the DependentNode and return true if we find the IndependentNode
	auto Visitor = [IndependentNode](FHandle UpStreamNode) { return IndependentNode == UpStreamNode; };
	if (DependentNode == IndependentNode || VistDependencies(DependentNode, Visitor) == true)
	{
		return true;
	}
	return false;
}


EGeometryFlowResult FGraph::CanAddConnection(FHandle FromNode, FString FromOutput, FHandle ToNode, FString ToInput) const
{
	int32 FromType = -1, ToType = -2;
	EGeometryFlowResult FromTypeResult = GetOutputTypeForNode(FromNode, FromOutput, FromType);
	if (!ensure(FromTypeResult == EGeometryFlowResult::Ok))
	{
		return FromTypeResult;
	}
	EGeometryFlowResult ToTypeResult = GetInputTypeForNode(ToNode, ToInput, ToType);
	if (!ensure(ToTypeResult == EGeometryFlowResult::Ok))
	{
		return ToTypeResult;
	}

	if (!ensure(FromType == ToType))
	{
		return EGeometryFlowResult::UnmatchedTypes;
	}

	const FConnection* Found = Algo::FindByPredicate(Connections, [&](const FConnection& Conn)
	{
		return ((Conn.ToNode == ToNode) && (Conn.ToInput == ToInput));
	});
	if (Found != nullptr)
	{
		return EGeometryFlowResult::InputAlreadyConnected;
	}

	// is the 'FromNode' already dependent on the 'ToNode'?  If so, the connection would create a cycle.
	if (IsDependent(FromNode, ToNode))
	{
		return EGeometryFlowResult::ConnectionRejectedCreatesCycle;
	};

	return EGeometryFlowResult::Ok;
}


EGeometryFlowResult FGraph::AddConnection(FHandle FromNode, FString FromOutput, FHandle ToNode, FString ToInput)
{
	EGeometryFlowResult Result = CanAddConnection(FromNode, FromOutput, ToNode, ToInput);

	if (Result == EGeometryFlowResult::Ok)
	{
		Connections.Add({ FromNode, FromOutput, ToNode, ToInput });
	}

	return Result;
}


EGeometryFlowResult FGraph::InferConnection(FHandle FromNodeHandle, FHandle ToNodeHandle)
{
	TSafeSharedPtr<FNode> FromNode = FindNode(FromNodeHandle);
	TSafeSharedPtr<FNode> ToNode = FindNode(ToNodeHandle);
	if (FromNode == nullptr || ToNode == nullptr)
	{
		ensure(false);
		return EGeometryFlowResult::NodeDoesNotExist;
	}

	if (FromNodeHandle == ToNodeHandle)
	{
		return EGeometryFlowResult::ConnectionRejectedCreatesCycle;
	}

	FString FromOutputName;
	FString ToInputName;
	int32 TotalMatchesFound = 0;
	FromNode->EnumerateOutputs([&](const FString& OutputName, const TUniquePtr<INodeOutput>& Output)
	{
		int32 OutputType = Output->GetDataType();
		ToNode->EnumerateInputs([&](const FString& InputName, const TUniquePtr<INodeInput>& Input)
		{
			// if input already has a connection, we cannot add another one and can skip it
			FConnection ExistingConnection;
			if (FindConnectionForInput(ToNodeHandle, InputName, ExistingConnection) != EGeometryFlowResult::Ok)
			{
				int32 InputType = Input->GetDataType();
				if (OutputType == InputType)
				{
					TotalMatchesFound++;
					FromOutputName = OutputName;
					ToInputName = InputName;
				}
			}
		});
	});
	ensure(TotalMatchesFound == 1);
	switch (TotalMatchesFound)
	{
	case 1:
		return AddConnection(FromNodeHandle, FromOutputName, ToNodeHandle, ToInputName);
	case 0:
		return EGeometryFlowResult::NoMatchesFound;
	default:
		return EGeometryFlowResult::MultipleMatchingAmbiguityFound;
	}
}

TSet<FGraph::FHandle> FGraph::GetSourceNodes() const
{
	TSet<FHandle> SourceNodes;
	for (const TMap<FHandle, FNodeInfo>::ElementType& IdNodePair : AllNodes)
	{
		if (IdNodePair.Value.Node->NodeInputs.Num() == 0)
		{
			SourceNodes.Add(FHandle{ IdNodePair.Key });
		}
	}
	return SourceNodes;
}

TSet<FGraph::FHandle> FGraph::GetNodesWithNoConnectedInputs() const
{
	TSet<FHandle> NoInputNodes;

	for (const TMap<FHandle, FNodeInfo>::ElementType& IdNodePair : AllNodes)
	{
		FHandle NodeHandle = IdNodePair.Key;
		TSafeSharedPtr<FNode> Node = IdNodePair.Value.Node;

		bool bAnyInputConnected = false;
		Node->EnumerateInputs([this, NodeHandle, &bAnyInputConnected](const FString& InputName, const TUniquePtr<INodeInput>& Input)
		{
			FConnection ConnectionOut;
			EGeometryFlowResult FindResult = FindConnectionForInput(NodeHandle, InputName, ConnectionOut);
			if (FindResult == EGeometryFlowResult::Ok)
			{
				bAnyInputConnected = true;
			}
		});

		if (!bAnyInputConnected)
		{
			NoInputNodes.Add(NodeHandle);
		}
	}

	return NoInputNodes;
}



EGeometryFlowResult FGraph::RemoveConnectionForInput(FHandle ToNode, FString ToInput)
{
	for (int i = 0; i < Connections.Num(); ++i)
	{
		FConnection& Connection = Connections[i];
		if (Connection.ToNode == ToNode && Connection.ToInput == ToInput)
		{
			Connections.RemoveAtSwap(i);
			return EGeometryFlowResult::Ok;
		}
	}

	return EGeometryFlowResult::ConnectionDoesNotExist;
}

EGeometryFlowResult FGraph::FindInputConnections(FHandle ToNode, TArray<FGraph::FConnection>& ConnectionsOut) const
{
	if (!AllNodes.Contains(ToNode)) return EGeometryFlowResult::NodeDoesNotExist;

	for (const FConnection& Connection : Connections)
	{
		if (Connection.ToNode == ToNode)
		{
			ConnectionsOut.Add(Connection);
		}
	}
	return EGeometryFlowResult::Ok;
}


EGeometryFlowResult FGraph::FindConnectionForInput(FHandle ToNode, FString ToInput, FConnection& ConnectionOut) const
{
	for (const FConnection& Connection : Connections)
	{
		if (Connection.ToNode == ToNode && Connection.ToInput == ToInput)
		{
			ConnectionOut = Connection;
			return EGeometryFlowResult::Ok;
		}
	}
	return EGeometryFlowResult::ConnectionDoesNotExist;
}


int32 FGraph::CountOutputConnections(FHandle FromNode, const FString& OutputName) const
{
	int32 Count = 0;
	for (const FConnection& Connection : Connections)
	{
		if (Connection.FromNode == FromNode && Connection.FromOutput == OutputName)
		{
			Count++;
		}
	}
	return Count;
}



TSafeSharedPtr<IData> FGraph::ComputeOutputData(
	FHandle NodeHandle, 
	FString OutputName, 
	TUniquePtr<FEvaluationInfo>& EvaluationInfo,
	bool bStealOutputData)
{
	TSafeSharedPtr<FNode> Node = FindNode(NodeHandle);
	check(Node);

	// figure out which upstream Connections/Inputs we need to compute this Output
	TArray<FString> Outputs;
	Outputs.Add(OutputName);
	TArray<FEvalRequirement> InputRequirements;
	Node->CollectRequirements({ OutputName }, InputRequirements);

	// this is the map of (InputName, Data) we will build up by pulling from the Connections
	FNamedDataMap DataIn;

	TArray<TFuture<void>> AsyncFutures;
	FRWLock DataInLock;

	// Collect data from those Inputs.
	// This will recursively call ComputeOutputData() on those (Node/Output) pairs
	for (int32 k = 0; k < InputRequirements.Num(); ++k)
	{
		FDataFlags DataFlags;
		const FString& InputName = InputRequirements[k].InputName;

		// find the connection for this input
		FConnection Connection;
		EGeometryFlowResult FoundResult = FindConnectionForInput(NodeHandle, InputName, Connection);

		if (FoundResult == EGeometryFlowResult::ConnectionDoesNotExist) 
		{
			TSafeSharedPtr<IData> DefaultData = Node->GetDefaultInputData(InputName);
			// TODO: Bubble this error up rather than crash
			checkf(DefaultData != nullptr, TEXT("Node \"%s\" input \"%s\" is not connected and has no default value"), *Node->GetIdentifier(), *InputName);
			DataInLock.WriteLock();
			DataIn.Add(InputName, DefaultData, DataFlags);
			DataInLock.WriteUnlock();
			continue;
		}

		ENodeCachingStrategy FromCachingStrategy = GetCachingStrategyForNode(Connection.FromNode);

		// If there is only one Connection from this upstream Output (ie to our Input), and the Node/Input 
		// can steal and transform that data, then we will do it
		int32 OutputUsageCount = CountOutputConnections(Connection.FromNode, Connection.FromOutput);
		bool bStealDataForInput = false;
		if (OutputUsageCount == 1 
			&& InputRequirements[k].InputFlags.bCanTransformInput
			&& FromCachingStrategy != ENodeCachingStrategy::AlwaysCache)
		{
			bStealDataForInput = true;
			DataFlags.bIsMutableData = true;
		}

		TFuture<void> Future = Async(GeometryFlowGraphAsyncExecTarget, 
									[this, &DataIn, &DataInLock, &InputName, DataFlags, Connection, &EvaluationInfo, bStealDataForInput] ()
		{
			// recursively fetch Data coming in to this Input via Connection
			TSafeSharedPtr<IData> UpstreamData = ComputeOutputData(Connection.FromNode, Connection.FromOutput, EvaluationInfo, bStealDataForInput);

			DataInLock.WriteLock();
			DataIn.Add(InputName, UpstreamData, DataFlags);
			DataInLock.WriteUnlock();
		});

		AsyncFutures.Emplace(MoveTemp(Future));
	}

	for (TFuture<void>& Future : AsyncFutures)
	{
		Future.Wait();
	}

	check(DataIn.GetNames().Num() == InputRequirements.Num());

	if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
	{
		return nullptr;
	}

	// evalute node
	FNamedDataMap DataOut;
	DataOut.Add(OutputName);
	
	TSafeSharedPtr<FRWLock> NodeLock = FindNodeLock(NodeHandle);
	NodeLock->WriteLock();
	Node->Evaluate(DataIn, DataOut, EvaluationInfo);
	NodeLock->WriteUnlock();

	if (EvaluationInfo)
	{
		EvaluationInfo->CountEvaluation(Node.Get());

		if (EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
		{
			return nullptr;
		}
	}

	// collect (and optionally take/steal) desired Output data
	TSafeSharedPtr<IData> Result = (bStealOutputData) ? 
		Node->StealOutput(OutputName) : DataOut.FindData(OutputName);

	check(Result);
	return Result;
}


bool FGraph::CanComputeOutput(FHandle NodeHandle, FString OutputName)
{
	TSafeSharedPtr<FNode> Node = FindNode(NodeHandle);
	if (!Node) return false;

	TArray<FEvalRequirement> InputRequirements;
	Node->CollectRequirements({ OutputName }, InputRequirements);

	const int NumInputRequirements = InputRequirements.Num();

	TArray<TFuture<bool>> AsyncFutures;
	AsyncFutures.Reserve(NumInputRequirements);

	// This will recursively call CanComputeOutput() on those (Node/Output) pairs
	for (int32 k = 0; k < NumInputRequirements; ++k)
	{
		const FString& InputName = InputRequirements[k].InputName;

		// find the connection for this input
		FConnection Connection;
		EGeometryFlowResult FoundResult = FindConnectionForInput(NodeHandle, InputName, Connection);

		if (FoundResult == EGeometryFlowResult::ConnectionDoesNotExist)
		{
			TSafeSharedPtr<IData> DefaultData = Node->GetDefaultInputData(InputName);
			if (!DefaultData) // the node input is not connected and has no default value;
			{
				return false;
			}
			continue;
		}


		TFuture<bool> Future = Async(GeometryFlowGraphAsyncExecTarget,
			[this, &InputName, Connection]()
			{
				// recursively look upstream from this Input via Connection
				const bool ValidUpstream = CanComputeOutput(Connection.FromNode, Connection.FromOutput);
				return ValidUpstream;

			});

		AsyncFutures.Emplace(MoveTemp(Future));
	}

	bool bAllInputsValid = true;
	for (TFuture<bool>& Future : AsyncFutures)
	{
		Future.Wait();
		bAllInputsValid = bAllInputsValid && Future.Get();
	}

	return bAllInputsValid;
}

bool FGraph::VistDependencies(FHandle NodeHandle, TFunctionRef<bool(FGraph::FHandle)> Visitor) const
{
	TSafeSharedPtr<FNode> Node = FindNode(NodeHandle);
	if (!Node) return false;

	TArray<FEvalRequirement> InputRequirements;
	Node->CollectAllRequirements(InputRequirements);

	const int NumInputRequirements = InputRequirements.Num();

	TArray<TFuture<bool>> AsyncFutures;
	AsyncFutures.Reserve(NumInputRequirements);

	// This will recursively call CanComputeOutput() on those (Node/Output) pairs
	for (int32 k = 0; k < NumInputRequirements; ++k)
	{
		const FString& InputName = InputRequirements[k].InputName;

		// find the connection for this input
		FConnection Connection;
		EGeometryFlowResult FoundResult = FindConnectionForInput(NodeHandle, InputName, Connection);

		if (FoundResult == EGeometryFlowResult::ConnectionDoesNotExist)
		{
			continue;
		}


		TFuture<bool> Future = Async(GeometryFlowGraphAsyncExecTarget,
			[this, &InputName, Connection, Visitor]()
			{
				if (Visitor(Connection.FromNode)) return true;

				// recursively look upstream from this Input via Connection
				const bool UpstreamResult = VistDependencies(Connection.FromNode, Visitor);
				return UpstreamResult;
			});

		AsyncFutures.Emplace(MoveTemp(Future));
	}

	bool bVisitorTerminated = false;
	for (TFuture<bool>& Future : AsyncFutures)
	{
		Future.Wait();
		bVisitorTerminated = bVisitorTerminated || Future.Get();
	}

	return bVisitorTerminated;
}




void FGraph::ConfigureCachingStrategy(ENodeCachingStrategy NewStrategy)
{
	if (NewStrategy != DefaultCachingStrategy && ensure(NewStrategy != ENodeCachingStrategy::Default) )
	{
		DefaultCachingStrategy = NewStrategy;

		// todo: clear caches if necessary?
	}
}


EGeometryFlowResult FGraph::SetNodeCachingStrategy(FHandle NodeHandle, ENodeCachingStrategy Strategy)
{
	FNodeInfo* Found = AllNodes.Find(NodeHandle);
	if (!ensure(Found != nullptr))
	{
		return EGeometryFlowResult::NodeDoesNotExist;
	}
	Found->CachingStrategy = Strategy;
	return EGeometryFlowResult::Ok;
}


FString FGraph::DebugDumpGraph(TFunction<bool(TSafeSharedPtr<FNode>)> IncludeNodeFn) const
{
	// Can be used by, e.g., https://csacademy.com/app/graph_editor/

	FString Out;

	// First, all node names
	for (const TPair<FHandle, FNodeInfo>& NodeHandleAndInfo : AllNodes)
	{
		if (!NodeHandleAndInfo.Value.Node)
		{ 
			return "Error"; 
		}
			
		if (!IncludeNodeFn(NodeHandleAndInfo.Value.Node))
		{ 
			continue; 
		}		

		FString NodeName = NodeHandleAndInfo.Value.Node->GetIdentifier();
		Out += NodeName + "\n";
	}

	// Second, connections by node name
	for (const FConnection& Connection : Connections)
	{
		FHandle FromHandle = Connection.FromNode;
		if (!AllNodes.Find(FromHandle) || !AllNodes.Find(FromHandle)->Node) 
		{ 
			return "Error"; 
		}

		TSafeSharedPtr<FNode> FromNode = AllNodes.Find(FromHandle)->Node;
		if (!IncludeNodeFn(FromNode))
		{
			continue;
		}

		FHandle ToHandle = Connection.ToNode;
		if (!AllNodes.Find(ToHandle) || !AllNodes.Find(ToHandle)->Node) 
		{ 
			return "Error"; 
		}

		TSafeSharedPtr<FNode> ToNode = AllNodes.Find(ToHandle)->Node;
		if (!IncludeNodeFn(ToNode))
		{ 
			continue; 
		}

		Out += FromNode->GetIdentifier() + " " + ToNode->GetIdentifier() + "\n";
	}

	return Out;
}


void FGraph::Serialize(FArchive& Ar)
{


	if (Ar.IsSaving())
	{
		float CurrentVersion = GetVersionID();
		Ar << CurrentVersion;
	}
	else
	{
		float ArchievedVersion;
		Ar << ArchievedVersion;

		if (!ensureMsgf(ArchievedVersion == GetVersionID(), TEXT("Unable to load Geometry Flow Graph Version %f"), ArchievedVersion))
		{
			// any future version of this graph will have to provide some mechanism for loading older versions
			return;
		}
	}
	// Data for serialization:  NodeCounter, Connections, AllNodes (note, AllNodeLocks aren't serialized but are reconstructed during IsLoading)
	Ar << NodeCounter;

	Ar << Connections;

	if (Ar.IsSaving())
	{
		int32 NumNodes = AllNodes.Num();
		Ar << NumNodes;

		for (TPair<FHandle, FNodeInfo>& Pair : AllNodes)
		{
			FHandle Handle = Pair.Key;
			FNodeInfo& NodeInfo = Pair.Value;
			FName NodeTypeName = NodeInfo.Node->GetType();
			FString Identifier = NodeInfo.Node->GetIdentifier();
			checkSlow(NodeTypeName == NodeInfo.NodeTypeName);
			Ar << Handle;
			Ar << NodeTypeName;
			Ar << NodeInfo.CachingStrategy;
			Ar << Identifier;

			NodeInfo.Node->Serialize(Ar);
		}
	}
	else if (Ar.IsLoading())
	{
		int32 NumNodes = 0;
		Ar << NumNodes;

		for (int32 i = 0; i < NumNodes; ++i)
		{
			FHandle Handle;
			FName NodeTypeName;
			ENodeCachingStrategy CachingStrategy;
			FString Identifier;

			Ar << Handle;
			Ar << NodeTypeName;
			Ar << CachingStrategy;
			Ar << Identifier;

			const ENodeAddResult AddResult = AddNodeOfType(NodeTypeName, Handle, Identifier, CachingStrategy); // automatically populates AllNodes and AllNodeLocks
			checkSlow(AddResult == ENodeAddResult::Success);
			ApplyToNode(Handle, [&Ar](FNode& Node) { Node.Serialize(Ar); });
		}
	}
}