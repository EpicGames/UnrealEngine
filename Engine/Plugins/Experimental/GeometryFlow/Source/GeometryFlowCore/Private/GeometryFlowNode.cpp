// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFlowNode.h"

using namespace UE::GeometryFlow;


void FNode::AddInput(FString Name, TUniquePtr<INodeInput>&& Input, TSafeSharedPtr<IData> DefaultData)
{
	// todo checks/etc
	if (DefaultData)
	{
		InputDefaultValues.Add(Name, DefaultData);
	}

	FNodeInputInfo NodeInputInfo;
	NodeInputInfo.Name = Name;
	NodeInputInfo.Input = MoveTemp(Input);
	NodeInputs.Add(MoveTemp(NodeInputInfo));
}
void FNode::AddOutput(FString Name, TUniquePtr<INodeOutput>&& Output)
{
	// todo checks/etc

	FNodeOutputInfo NodeOutputInfo;
	NodeOutputInfo.Name = Name;
	NodeOutputInfo.Output = MoveTemp(Output);
	NodeOutputs.Add(MoveTemp(NodeOutputInfo));
}


EGeometryFlowResult FNode::GetInputType(FString Name, int32& TypeOut) const
{
	int32 Index = NodeInputs.IndexOfByPredicate([&](const FNodeInputInfo& Input) { return Input.Name == Name; });
	if (Index != INDEX_NONE)
	{
		TypeOut = NodeInputs[Index].Input->GetDataType();
		return EGeometryFlowResult::Ok;
	}
	return EGeometryFlowResult::InputDoesNotExist;
}

FNodeInputFlags FNode::GetInputFlags(FString InputName) const
{
	int32 Index = NodeInputs.IndexOfByPredicate([&](const FNodeInputInfo& Input) { return Input.Name == InputName; });
	if (ensure(Index != INDEX_NONE))
	{
		return NodeInputs[Index].Input->GetInputFlags();
	}
	return FNodeInputFlags();
}

EGeometryFlowResult FNode::GetOutputType(FString Name, int32& TypeOut) const
{
	int32 Index = NodeOutputs.IndexOfByPredicate([&](const FNodeOutputInfo& Output) { return Output.Name == Name; });
	if (Index != INDEX_NONE)
	{
		TypeOut = NodeOutputs[Index].Output->GetDataType();
		return EGeometryFlowResult::Ok;
	}
	return EGeometryFlowResult::OutputDoesNotExist;
}


void FNode::EnumerateInputs(TFunctionRef<void(const FString& Name, const TUniquePtr<INodeInput>& Input)> EnumerateFunc) const
{
	for (const FNodeInputInfo& InputInfo : NodeInputs)
	{
		EnumerateFunc(InputInfo.Name, InputInfo.Input);
	}

}

void FNode::EnumerateOutputs(TFunctionRef<void(const FString& Name, const TUniquePtr<INodeOutput>& Output)> EnumerateFunc) const
{
	for (const FNodeOutputInfo& OutputInfo : NodeOutputs)
	{
		EnumerateFunc(OutputInfo.Name, OutputInfo.Output);
	}
}


bool FNode::IsOutputAvailable(FString OutputName) const
{
	for (const FNodeOutputInfo& OutputInfo : NodeOutputs)
	{
		if (OutputInfo.Name == OutputName)
		{
			return OutputInfo.Output->HasCachedOutput();
		}
	}
	ensure(false);		// not found
	return false;
}


TSafeSharedPtr<IData> FNode::StealOutput(FString OutputName)
{
	for (const FNodeOutputInfo& OutputInfo : NodeOutputs)
	{
		if (OutputInfo.Name == OutputName)
		{
			return OutputInfo.Output->StealOutput();
		}
	}
	ensure(false);
	return TSafeSharedPtr<IData>();
}


void FNode::CollectRequirements(const TArray<FString>& Outputs, TArray<FEvalRequirement>& RequiredInputsOut)
{
	RequiredInputsOut.Reset();

	int32 FoundOutputs = 0;
	for (const FNodeOutputInfo& OutputInfo : NodeOutputs)
	{
		if (Outputs.Contains(OutputInfo.Name))
		{
			FoundOutputs++;
		}
	}
	if (ensure(FoundOutputs != 0) == false)
	{
		return;
	}

	for ( const FNodeInputInfo & InputInfo : NodeInputs )
	{
		FEvalRequirement Requirement(InputInfo.Name, InputInfo.Input->GetInputFlags());
		RequiredInputsOut.Add(Requirement);
	}
}


void FNode::CollectAllRequirements(TArray<FEvalRequirement>& RequiredInputsOut)
{
	RequiredInputsOut.Reset();
	for (const FNodeInputInfo& InputInfo : NodeInputs)
	{
		FEvalRequirement Requirement(InputInfo.Name, InputInfo.Input->GetInputFlags());
		RequiredInputsOut.Add(Requirement);
	}
}


void FNode::ConfigureInputFlags(const FString& InputName, FNodeInputFlags Flags)
{
	int32 Index = NodeInputs.IndexOfByPredicate([&](const FNodeInputInfo& Input) { return Input.Name == InputName; });
	if (ensure(Index != INDEX_NONE))
	{
		return NodeInputs[Index].Input->SetInputFlags(Flags);
	}
}


bool FNode::IsInputDirty(FString Name, int32 NewTimestamp) const
{
	int32 Index = NodeInputs.IndexOfByPredicate([&](const FNodeInputInfo& Input) { return Input.Name == Name; });
	if (ensure(Index != INDEX_NONE))
	{
		return NodeInputs[Index].LastTimestamp != NewTimestamp;
	}
	return true;
}

bool FNode::CheckIsInputDirtyAndUpdate(FString Name, int32 NewTimestamp)
{
	bool bDirty = true;
	int32 Index = NodeInputs.IndexOfByPredicate([&](const FNodeInputInfo& Input) { return Input.Name == Name; });
	if (ensure(Index != INDEX_NONE))
	{
		bDirty = NodeInputs[Index].LastTimestamp != NewTimestamp;
		if (bDirty)
		{
			NodeInputs[Index].LastTimestamp = NewTimestamp;
		}
	}
	return bDirty;
}


void FNode::UpdateInputTimestamp(FString InputName, int32 NewTimestamp)
{
	int32 Index = NodeInputs.IndexOfByPredicate([&](const FNodeInputInfo& Input) { return Input.Name == InputName; });
	if (ensure(Index != INDEX_NONE))
	{
		NodeInputs[Index].LastTimestamp = NewTimestamp;
	}
}

void FNode::SetOutput(
	const FString& OutputName,
	TSafeSharedPtr<IData> NewData
)
{
	int32 Index = NodeOutputs.IndexOfByPredicate([&](const FNodeOutputInfo& Output) { return Output.Name == OutputName; });
	if (ensure(Index != INDEX_NONE))
	{
		NodeOutputs[Index].Output->UpdateOutput(NewData);
	}
}


TSafeSharedPtr<IData> FNode::GetOutput(const FString& OutputName) const
{
	int32 Index = NodeOutputs.IndexOfByPredicate([&](const FNodeOutputInfo& Output) { return Output.Name == OutputName; });
	if (ensure(Index != INDEX_NONE))
	{
		return NodeOutputs[Index].Output->GetOutput();
	}
	return nullptr;
}

void FNode::ClearOutput(const FString& OutputName)
{
	int32 Index = NodeOutputs.IndexOfByPredicate([&](const FNodeOutputInfo& Output) { return Output.Name == OutputName; });
	if (ensure(Index != INDEX_NONE))
	{
		NodeOutputs[Index].Output->ClearOutputCache();
	}
}

void FNode::ClearAllOutputs()
{
	for (FNodeOutputInfo& OutputInfo : NodeOutputs)
	{
		OutputInfo.Output->ClearOutputCache();
	}
}

TSafeSharedPtr<IData> FNode::FindAndUpdateInputForEvaluate(const FString& InputName, const FNamedDataMap& DatasIn,
	bool& bAccumModifiedOut, bool& bAccumValidOut)
{
	if (ensure(DatasIn.Contains(InputName)) == false)
	{
		bAccumValidOut = false;
		return TSafeSharedPtr<IData>();
	}
	else
	{
		TSafeSharedPtr<IData> Argument = DatasIn.FindData(InputName);
		bool bInputModified = CheckIsInputDirtyAndUpdate(InputName, Argument->GetTimestamp());
		bAccumModifiedOut |= bInputModified;
		return Argument;
	}
}







FEvaluationInfo::FEvaluationInfo()
{
	EvaluationsCount = 0;
	ComputesCount = 0;
}

void FEvaluationInfo::Reset()
{
	EvaluationsCount = 0;
	ComputesCount = 0;
}

void FEvaluationInfo::CountEvaluation(FNode* Node)
{
	EvaluationsCount++;
}

void FEvaluationInfo::CountCompute(FNode* Node)
{
	ComputesCount++;
}