// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFlowNodeFactory.h"
#include "Misc/LazySingleton.h"


UE::GeometryFlow::FNodeFactory& UE::GeometryFlow::FNodeFactory::GetInstance()
{
	return TLazySingleton< UE::GeometryFlow::FNodeFactory >::Get();
};

void UE::GeometryFlow::FNodeFactory::TearDown()
{
	TLazySingleton< UE::GeometryFlow::FNodeFactory >::TearDown();
};