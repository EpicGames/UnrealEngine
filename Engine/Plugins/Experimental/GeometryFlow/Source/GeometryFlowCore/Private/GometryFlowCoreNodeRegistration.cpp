// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFlowCoreNodeRegistration.h"
#include "GeometryFlowNodeFactory.h"
#include "GeometryFlowCoreNodes.h"

#include "BaseNodes/SwitchNode.h"
#include "MathNodes/ArithmeticNodes.h"


namespace UE
{
namespace GeometryFlow
{
#define GEOMETRY_FLOW_QUOTE(x) #x

// register node of type F{TypeName}SourceNode with FName {TypeName}Source
#define GEOMETRYFLOW_REGISTER_BASIC_TYPES(TypeName) \
	FNodeFactory::GetInstance().RegisterType<F##TypeName##SourceNode>(FString(GEOMETRY_FLOW_QUOTE(TypeName##Source)), \
																	  CategoryName);
// register node of type F{Typename}StructSourceNode with FName {TypeName}StructSource
#define GEOMETRYFLOW_REGISTER_BASIC_USTRUCT_TYPES(TypeName) \
	FNodeFactory::GetInstance().RegisterType<F##TypeName##Struct##SourceNode>(FString(GEOMETRY_FLOW_QUOTE(TypeName##Struct##Source)), \
																	  CategoryName);

	// Register Core Nodes with the node factory
	void FCoreNodeRegistration::RegisterNodes()
	{	
		FString CategoryName("Basic Types");

		GEOMETRYFLOW_REGISTER_BASIC_TYPES(Int32);
		GEOMETRYFLOW_REGISTER_BASIC_TYPES(Float);
		GEOMETRYFLOW_REGISTER_BASIC_TYPES(Double);
		GEOMETRYFLOW_REGISTER_BASIC_TYPES(Vector3f);
		GEOMETRYFLOW_REGISTER_BASIC_TYPES(Vector3d);
		GEOMETRYFLOW_REGISTER_BASIC_TYPES(Name);

		GEOMETRYFLOW_REGISTER_BASIC_USTRUCT_TYPES(Int32);
		GEOMETRYFLOW_REGISTER_BASIC_USTRUCT_TYPES(Float);
		GEOMETRYFLOW_REGISTER_BASIC_USTRUCT_TYPES(Double);
		GEOMETRYFLOW_REGISTER_BASIC_USTRUCT_TYPES(Vector3f);
		GEOMETRYFLOW_REGISTER_BASIC_USTRUCT_TYPES(Vector3d);
		GEOMETRYFLOW_REGISTER_BASIC_USTRUCT_TYPES(Name);

		FNodeFactory::GetInstance().RegisterType< FAddFloatNode >(FString("AddFloat"), CategoryName);
	}


#undef GEOMETRYFLOW_REGISTER_BASIC_TYPES_WUI
#undef GEOMETRYFLOW_REGISTER_BASIC_TYPES
#undef GEOMETRY_FLOW_QUOTE
}
}