// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "GeometryFlowMovableData.h"


namespace UE
{
namespace GeometryFlow
{



template<typename T, int StorageTypeIdentifier>
class TBinaryOpLambdaNode : public FNode
{
protected:
	using DataType = TMovableData<T, StorageTypeIdentifier>;

	TUniqueFunction<T(const T& A, const T& B)> OpFunc;

public:
	static const FString InParamArg1() { return TEXT("Operand1"); }
	static const FString InParamArg2() { return TEXT("Operand2"); }
	static const FString OutParamResult() { return TEXT("Result"); }

protected:
	TBinaryOpLambdaNode()
	{
		AddInput(InParamArg1(), MakeUnique<TBasicNodeInput<T, StorageTypeIdentifier>>());
		AddInput(InParamArg2(), MakeUnique<TBasicNodeInput<T, StorageTypeIdentifier>>());

		AddOutput(OutParamResult(), MakeUnique<TBasicNodeOutput<T, StorageTypeIdentifier>>());
	}

public:
	TBinaryOpLambdaNode(TUniqueFunction<T(const T& A, const T& B)>&& OpFuncIn) : TBinaryOpLambdaNode()
	{
		OpFunc = MoveTemp(OpFuncIn);
	}

	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamResult())))
		{
			bool bAnyInputsModified = false, bAllInputsValid = true;
			TSafeSharedPtr<IData> Arg1 = FindAndUpdateInputForEvaluate(InParamArg1(), DatasIn, bAnyInputsModified, bAllInputsValid);
			TSafeSharedPtr<IData> Arg2 = FindAndUpdateInputForEvaluate(InParamArg2(), DatasIn, bAnyInputsModified, bAllInputsValid);
			if (bAllInputsValid)
			{
				if (bAnyInputsModified)
				{
					T A, B;
					Arg1->GetDataCopy(A, StorageTypeIdentifier);
					Arg2->GetDataCopy(B, StorageTypeIdentifier);

					T BinaryOpResult = OpFunc(A, B);

					TSafeSharedPtr<DataType> Result = MakeSafeShared<DataType>();
					Result->SetData(BinaryOpResult);
					SetOutput(OutParamResult(), Result);

					EvaluationInfo->CountCompute(this);
				}
				DatasOut.SetData(OutParamResult(), GetOutput(OutParamResult()) );
			}
		}
	}
};






}	// end namespace GeometryFlow
}	// end namespace UE
