// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "GeometryFlowMovableData.h"


namespace UE
{
namespace GeometryFlow
{


/**
 * TSwitchNode selects one of N inputs and provides it as an Output. This allows for a minimal amount of 
 * branching / control-flow in a fixed GeometryFlow Graph. Only the active Input is evaluated.
 * 
 * The number of inputs is defined in the template type, eg TSwitchNode<FDynamicMesh3, 3, (int)EMeshProcessingDataTypes::DynamicMesh> 
 * is a switch node with 3 possible inputs. Dynamic number of inputs cannot be easily supported without
 * changes at the graph evaluation level.
 * 
 * Use UpdateSwitchValue() to select the active switch Input
 * 
 */
template<typename T, int NumInputs, int StorageTypeIdentifier>
class TSwitchNode : public FNode
{

	static constexpr int NodeVersion = 1;
public:
	typedef FNode Super;
	static FName StaticType() 
	{ 
		FString NodeTypeName = FString::Printf(TEXT("FSwitchNode%02dInputs%02dType"), NumInputs, StorageTypeIdentifier);
		return FName(*NodeTypeName); 
	}
	virtual FName GetType() const override { return StaticType(); }
	virtual bool  IsA(FName InType) const override { return InType.ToString().Equals(StaticType().ToString()) || Super::IsA(InType); }
	virtual int32 GetVersionID() const override { return NodeVersion; }

protected:
	using DataContainerType = TMovableData<T, StorageTypeIdentifier>;

	int32 SwitchIndex = 0;

public:
	static const FString InParamValue(int32 InputNum)
	{
		return FString::Printf(TEXT("Input%d"), InputNum);
	}

	static const FString OutParamValue() { return TEXT("Output"); }

public:
	TSwitchNode()
	{
		// add N input nodes
		for (int32 k = 0; k < NumInputs; ++k)
		{
			AddInput(InParamValue(k), MakeUnique<TBasicNodeInput<T, StorageTypeIdentifier>>());
		}
		 
		AddOutput(OutParamValue(), MakeUnique<TBasicNodeOutput<T, StorageTypeIdentifier>>());

		// assume we can steal input and pass to output
		for (int32 k = 0; k < NumInputs; ++k)
		{
			ConfigureInputFlags(InParamValue(k), FNodeInputFlags::Transformable());
		}

		SwitchIndex = 0;
	}

	/**
	 * This function is used to update the Input Index, ie select which Output will be provided. 
	 * This function always Invalidates the Output if the Index was changed.
	 */
	virtual void UpdateSwitchInputIndex(const int32& NewSwitchIndex)
	{
		if (NewSwitchIndex != SwitchIndex)
		{
			SwitchIndex = NewSwitchIndex;

			// clear output to force recompute...
			ClearOutput(OutParamValue());
		}
	}

	/**
	 * Determine which Inputs are required for the desired Output. This replaces the default behavior ("all inputs")
	 * with only the selected Input returned as a Requirement, which in turn allows the Graph Evaluator to skip evaluation 
	 * of the other inputs. 
	 */
	virtual void CollectRequirements(const TArray<FString>& Outputs, TArray<FEvalRequirement>& RequiredInputsOut) override
	{
		RequiredInputsOut.Reset();

		int32 FoundOutputs = 0;
		for (const FNodeOutputInfo& OutputInfo : NodeOutputs)
		{
			if (Outputs.Contains(OutputInfo.Name))
			{
				FoundOutputs++;
			}
		}
		if (ensure(FoundOutputs != 0) == false)
		{
			return;
		}

		// only the active input is returned as a requirement
		FString ActiveInput = InParamValue(SwitchIndex);
		for ( const FNodeInputInfo & InputInfo : NodeInputs )
		{
			if (InputInfo.Name == ActiveInput)
			{
				FEvalRequirement Requirement(InputInfo.Name, InputInfo.Input->GetInputFlags());
				RequiredInputsOut.Add(Requirement);
			}
		}	
	}


	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamValue())))
		{
			bool bAllInputsValid = true;
			bool bRecomputeRequired = ( IsOutputAvailable(OutParamValue()) == false );

			// get the switch index and clamp it
			int UseSwitchIndex = SwitchIndex;
			ensure(UseSwitchIndex >= 0 && UseSwitchIndex < NumInputs);
			UseSwitchIndex = FMath::Clamp(UseSwitchIndex, 0, NumInputs);

			// select the specified switch input argument
			TSafeSharedPtr<IData> InputArg = FindAndUpdateInputForEvaluate(InParamValue(UseSwitchIndex), DatasIn, bRecomputeRequired, bAllInputsValid);
			if (bAllInputsValid)
			{
				if (bRecomputeRequired)
				{
					bool bIsDataMutable = DatasIn.GetDataFlags(InParamValue(UseSwitchIndex)).bIsMutableData;

					T NewData;
					if (bIsDataMutable)
					{
						InputArg->GiveTo<T>(NewData, StorageTypeIdentifier);

					}
					else
					{
						InputArg->GetDataCopy(NewData, StorageTypeIdentifier);
					}

					// store new result
					TSafeSharedPtr<DataContainerType> Result = MakeSafeShared<DataContainerType>();
					Result->MoveData(MoveTemp(NewData));
					SetOutput(OutParamValue(), Result);

					EvaluationInfo->CountCompute(this);
				}
				DatasOut.SetData(OutParamValue(), GetOutput(OutParamValue()));
			}
		}
	}
};

}	// end namespace GeometryFlow
}	// end namespace UE
