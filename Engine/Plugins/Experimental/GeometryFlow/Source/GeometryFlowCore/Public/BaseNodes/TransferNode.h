// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "GeometryFlowMovableData.h"


namespace UE
{
namespace GeometryFlow
{


template<typename T, int StorageTypeIdentifier>
class TTransferNode : public FNode
{
protected:
	using DataContainerType = TMovableData<T, StorageTypeIdentifier>;

public:
	static const FString InParamValue() { return TEXT("Input"); }
	static const FString OutParamValue() { return TEXT("Output"); }

public:
	TTransferNode()
	{
		AddInput(InParamValue(), MakeUnique<TBasicNodeInput<T, StorageTypeIdentifier>>());
		AddOutput(OutParamValue(), MakeUnique<TBasicNodeOutput<T, StorageTypeIdentifier>>());

		// we can steal input and pass to output
		ConfigureInputFlags(InParamValue(), FNodeInputFlags::Transformable());
	}

	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamValue())))
		{
			bool bAllInputsValid = true;
			bool bRecomputeRequired = ( IsOutputAvailable(OutParamValue()) == false );
			TSafeSharedPtr<IData> InputArg = FindAndUpdateInputForEvaluate(InParamValue(), DatasIn, bRecomputeRequired, bAllInputsValid);
			if (bAllInputsValid)
			{
				if (bRecomputeRequired)
				{
					bool bIsDataMutable = DatasIn.GetDataFlags(InParamValue()).bIsMutableData;

					T NewData;
					if (bIsDataMutable)
					{
						InputArg->GiveTo<T>(NewData, StorageTypeIdentifier);

					}
					else
					{
						InputArg->GetDataCopy(NewData, StorageTypeIdentifier);
					}

					// store new result
					TSafeSharedPtr<DataContainerType> Result = MakeSafeShared<DataContainerType>();
					Result->MoveData(MoveTemp(NewData));
					SetOutput(OutParamValue(), Result);

					EvaluationInfo->CountCompute(this);
				}
				DatasOut.SetData(OutParamValue(), GetOutput(OutParamValue()));
			}
		}
	}
};




}	// end namespace GeometryFlow
}	// end namespace UE
