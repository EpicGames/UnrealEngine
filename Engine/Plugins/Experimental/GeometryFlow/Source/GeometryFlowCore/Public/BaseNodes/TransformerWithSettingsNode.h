// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "GeometryFlowMovableData.h"


namespace UE
{
namespace GeometryFlow
{


template<
	typename InputType, 
	int InputDataTypeIdentifier,
	typename SettingsType, 
	int SettingsDataTypeIdentifier,
	typename OutputType,
	int OutputDataTypeIdentifier>
class TTransformerWithSettingsNode : public FNode
{
protected:
	using OutputDataType = TMovableData<OutputType, OutputDataTypeIdentifier>;

public:
	static const FString InParamInput() { return TEXT("Input"); }
	static const FString InParamSettings() { return TEXT("Settings"); }
	static const FString OutParamResult() { return TEXT("Result"); }

public:
	TTransformerWithSettingsNode()
	{
		AddInput(InParamInput(), MakeUnique<TBasicNodeInput<InputType, InputDataTypeIdentifier>>());
		AddInput(InParamSettings(), MakeUnique<TBasicNodeInput<SettingsType, SettingsDataTypeIdentifier>>());

		AddOutput(OutParamResult(), MakeUnique<TBasicNodeOutput<OutputType, OutputDataTypeIdentifier>>());
	}


protected:

	//
	// TTransformerWithSettingsNode API that subclasses must/can implement
	//

	virtual void ComputeOutput(
		const FNamedDataMap& DatasIn,
		const SettingsType& SettingsIn,
		const InputType& InputIn,
		OutputType& OutputOut)
	{
		check(false);		// should never be here, subclass needs to override
	}

	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid)
	{
		// none
	}


public:


	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamResult())))
		{
			bool bAllInputsValid = true;
			bool bRecomputeRequired = (IsOutputAvailable(OutParamResult()) == false);
			TSafeSharedPtr<IData> InputArg = FindAndUpdateInputForEvaluate(InParamInput(), DatasIn, bRecomputeRequired, bAllInputsValid);
			TSafeSharedPtr<IData> SettingsArg = FindAndUpdateInputForEvaluate(InParamSettings(), DatasIn, bRecomputeRequired, bAllInputsValid);
			CheckAdditionalInputs(DatasIn, bRecomputeRequired, bAllInputsValid);
			if (bAllInputsValid)
			{
				if (bRecomputeRequired)
				{
					const InputType& SourceData = InputArg->GetDataConstRef<InputType>(InputDataTypeIdentifier);

					// always make a copy of settings
					SettingsType Settings;
					SettingsArg->GetDataCopy(Settings, SettingsDataTypeIdentifier);

					// run processing
					OutputType ResultData;
					ComputeOutput(DatasIn, Settings, SourceData, ResultData);

					// store new result
					TSafeSharedPtr<OutputDataType> Result = MakeSafeShared<OutputDataType>();
					Result->MoveData(MoveTemp(ResultData));
					SetOutput(OutParamResult(), Result);

					EvaluationInfo->CountCompute(this);
				}
				DatasOut.SetData(OutParamResult(), GetOutput(OutParamResult()));
			}
		}
	}



};






}	// end namespace GeometryFlow
}	// end namespace UE
