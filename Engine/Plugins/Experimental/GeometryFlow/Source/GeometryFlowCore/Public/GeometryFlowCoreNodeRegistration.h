// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE
{
namespace GeometryFlow
{

	struct FCoreNodeRegistration
	{
		// function that registers the core nodes defined in GeometryFlowCore (mostly POD source nodes)
		GEOMETRYFLOWCORE_API static void RegisterNodes();
	};
}
}