// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowTSourceNode.h"
#include "GeometryFlowTypes.h"

#include "GeometryFlowCoreNodes.generated.h"

USTRUCT()
struct FInt32Setting
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EDataTypes::Integer);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 Value = 0;	
};

USTRUCT()
struct FFloatSetting
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EDataTypes::Float);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float Value = 0;	
};

USTRUCT()
struct FDoubleSetting
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EDataTypes::Double);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double Value = 0;	
};	


USTRUCT()
struct FVector3fSetting
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EDataTypes::Vector3f);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FVector3f Value = FVector3f(0, 0, 0);	
};	

USTRUCT()
struct FVector3dSetting
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EDataTypes::Vector3d);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FVector3d Value = FVector3d(0, 0, 0);	
};	


USTRUCT()
struct FNameSetting
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EDataTypes::Name);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FName Value;	
};	

namespace UE
{
namespace GeometryFlow
{
/**
 * Declare basic types for math
 */
GEOMETRYFLOW_DECLARE_BASIC_TYPES(Int32, int, (int)EDataTypes::Integer, 1)
GEOMETRYFLOW_DECLARE_BASIC_TYPES(Float, float, (int)EDataTypes::Float, 1)
GEOMETRYFLOW_DECLARE_BASIC_TYPES(Double, double, (int)EDataTypes::Double, 1)
GEOMETRYFLOW_DECLARE_BASIC_TYPES(Vector3f, FVector3f, (int)EDataTypes::Vector3f, 1)
GEOMETRYFLOW_DECLARE_BASIC_TYPES(Vector3d, FVector3d, (int)EDataTypes::Vector3d, 1)
GEOMETRYFLOW_DECLARE_BASIC_TYPES(Name, FName, (int)EDataTypes::Name, 1)

/**
* basic math types wrapped in USTRUCTs so they can be exposed as details
*/
GEOMETRYFLOW_DECLARE_BASIC_USTRUCT_TYPES(Int32, FInt32Setting, (int)EDataTypes::IntegerStruct, 1)
GEOMETRYFLOW_DECLARE_BASIC_USTRUCT_TYPES(Float, FFloatSetting, (int)EDataTypes::FloatStruct, 1)
GEOMETRYFLOW_DECLARE_BASIC_USTRUCT_TYPES(Double, FDoubleSetting, (int)EDataTypes::DoubleStruct, 1)
GEOMETRYFLOW_DECLARE_BASIC_USTRUCT_TYPES(Vector3f, FVector3fSetting, (int)EDataTypes::Vector3fStruct, 1)
GEOMETRYFLOW_DECLARE_BASIC_USTRUCT_TYPES(Vector3d, FVector3dSetting, (int)EDataTypes::Vector3dStruct, 1)
GEOMETRYFLOW_DECLARE_BASIC_USTRUCT_TYPES(Name, FNameSetting, (int)EDataTypes::NameStruct, 1)


}	// end namespace GeometryFlow
}	// end namespace UE
