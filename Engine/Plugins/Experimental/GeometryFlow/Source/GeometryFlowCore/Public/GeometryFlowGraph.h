// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "GeometryFlowNodeFactory.h"
#include "Util/ProgressCancel.h"
#include "Serialization/Archive.h"

#define UE_API GEOMETRYFLOWCORE_API

class FProgressCancel;

GEOMETRYFLOWCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogGeometryFlowGraph, Display, All);

namespace UE
{
namespace GeometryFlow
{



// TODO: this needs more work
	// need option to opportunistically cache, ie on multi-use of output, or when we can't re-use anyway
	//   (this is currently what NeverCache does?)

enum class ENodeCachingStrategy
{
	Default = 0,
	AlwaysCache = 1,
	NeverCache = 2
};


//
// TODO: 
// - internal FNode pointers can be unique ptr?
// - parallel evaluation at graph level (pre-pass to collect references?)
//


class FGraph
{

	// version used in serialization 
	static constexpr float SerializationVersion = 1.0;

public:

	UE_API ~FGraph();

	float GetVersionID() const { return FGraph::SerializationVersion;}

	struct FHandle
	{
		static const int32 InvalidHandle = -1;
		int32 Identifier = InvalidHandle;
		bool operator==(const FHandle& OtherHandle) const { return Identifier == OtherHandle.Identifier; }

		/**
		* Serialization operator for FHandle
		*
		* @parm Ar Archive to serialize with.
		* @param H  Handle to serialize.
		* @returns Passing down serializing archive
		*/
		friend FArchive& operator<<(FArchive& Ar, FHandle& H)
		{
			H.Serialize(Ar);
			return Ar;
		}

		/** Serialize FHandle to an archive*/
		void Serialize(FArchive& Ar)
		{
			Ar << Identifier;
		}
	};

	friend uint32 GetTypeHash(FHandle Handle);

	struct FConnection
	{
		FHandle FromNode;
		FString FromOutput;
		FHandle ToNode;
		FString ToInput;

		/**
		* Serialization operator for FConnection
		*
		* @parm Ar Archive to serialize with.
		* @param C  Connection to serialize.
		* @returns Passing down serializing archive
		*/
		friend FArchive& operator<<(FArchive& Ar, FConnection& C)
		{
			C.Serialize(Ar);
			return Ar;
		}

		/** Serialize FHandle to an archive*/
		void Serialize(FArchive& Ar)
		{
			Ar << FromNode;
			Ar << FromOutput;
			Ar << ToNode;
			Ar << ToInput;
		}

		bool operator==(const FConnection& Other) const
		{
			return (FromNode == Other.FromNode)
				&& (FromOutput == Other.FromOutput)
				&& (ToNode == Other.ToNode)
				&& (ToInput == Other.ToInput);
		}
	};

	// return @false if the node type is not registered
	template <typename NodeType>
	bool CanAddNodeOfType()
	{
		return CanAddNodeOfType(NodeType::StaticType());
	}

	// Add a node of specified type to the graph.  An invalid FHandle will be returned if failed (e.g. node type not registered)
	template <typename NodeType>
	FHandle AddNodeOfType(const FString& Identifier = FString(""),
		ENodeCachingStrategy CachingStrategy = ENodeCachingStrategy::Default)
	{
		return AddNodeOfType(NodeType::StaticType(), Identifier, CachingStrategy);
	}

	// Add a node of specified type to the graph.  An invalid FHandle will be returned if failed (e.g. node type not registered)
	UE_API FHandle AddNodeOfType(
		const FName TypeName,
		const FString& Identifier = FString(""),
		ENodeCachingStrategy CachingStrategy = ENodeCachingStrategy::Default);

	// Note: once removed, any external handles to this node will be stale. 
	void RemoveNode(const FHandle Handle)
	{
		RemoveNodeConnections(Handle);

		AllNodes.Remove(Handle);
		AllNodeLocks.Remove(Handle);
	}

	// returns true if DependentNode is dependent on IndependentNode in the graph.
	// special cases:
	// will return false if either handle is invalid,
	// if both handles are the same (and valid) it will return true.
	UE_API bool IsDependent(FHandle DependentNode, FHandle IndependentNode) const;

	// returns EGeometryFlowResult::Ok if the Connection can be added, otherwise returns the reason for failure
	UE_API EGeometryFlowResult CanAddConnection(FHandle FromNode, FString FromOutput, FHandle ToNode, FString ToInput) const;

	UE_API EGeometryFlowResult AddConnection(FHandle FromNode, FString FromOutput, FHandle ToNode, FString ToInput);

	UE_API EGeometryFlowResult InferConnection(FHandle FromNode, FHandle ToNode);
	
	// find all the input connections for the specified node.
	UE_API EGeometryFlowResult FindInputConnections(FHandle ToNode, TArray<FGraph::FConnection>& ConnectionsOut) const;

	// remove input connections to the specifed node
	void RemoveNodeInputConnections(FHandle NodeHandle)
	{
		int32 index = Connections.IndexOfByPredicate([&](const FConnection& Conn) { return (Conn.ToNode == NodeHandle); });
		while (index != INDEX_NONE)
		{
			Connections.RemoveAtSwap(index);
			index = Connections.IndexOfByPredicate([&](const FConnection& Conn) { return (Conn.ToNode == NodeHandle); });
		}
	}

	// remove all output connections to the specified node.
	void RemoveNodeOutputConnections(FHandle NodeHandle)
	{
		int32 index = Connections.IndexOfByPredicate([&](const FConnection& Conn) { return (Conn.FromNode == NodeHandle); });
		while (index != INDEX_NONE)
		{
			Connections.RemoveAtSwap(index);
			index = Connections.IndexOfByPredicate([&](const FConnection& Conn) { return (Conn.FromNode == NodeHandle); });
		}
	}

	// remove all connections to the specified node.
	void RemoveNodeConnections(FHandle NodeHandle)
	{
		RemoveNodeInputConnections(NodeHandle);
		RemoveNodeOutputConnections(NodeHandle);
	}

	UE_API EGeometryFlowResult RemoveConnectionForInput(FHandle ToNode, FString ToInput);

	UE_API TSet<FHandle> GetSourceNodes() const;
	UE_API TSet<FHandle> GetNodesWithNoConnectedInputs() const;

	template<typename T>
	EGeometryFlowResult EvaluateResult(
		FHandle Node,
		FString OutputName,
		T& Storage,
		int32 StorageTypeIdentifier,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo,
		bool bTryTakeResult)
	{
		EvaluateLock.Lock();
		EGeometryFlowResult Result;
		Result = EvaluateResultInternal(Node, OutputName, Storage, StorageTypeIdentifier, EvaluationInfo, bTryTakeResult);
		EvaluateLock.Unlock();
		return Result;
	}

	bool CanEvaluate(
		FHandle Node,
		FString OutputName
	)
	{
		return CanComputeOutput(Node, OutputName);
	}

	// Applies the functor to the node as derived type if possible.
	template<typename NodeType>
	EGeometryFlowResult ApplyToNodeOfType(
		FHandle NodeHandle, 
		TFunctionRef<void(NodeType&)> ApplyFunc)
	{
		TSafeSharedPtr<FNode> FoundNode = FindNode(NodeHandle);
		if (FoundNode)
		{
			NodeType* Value = static_cast<NodeType*>(FoundNode.Get());
			ApplyFunc(*Value);
			return EGeometryFlowResult::Ok;
		}
		return EGeometryFlowResult::NodeDoesNotExist;
	}

	// Applies the functor to the node as FNode.
	EGeometryFlowResult ApplyToNode(
		FHandle NodeHandle,
		TFunctionRef<void(FNode&)> ApplyFunc)
	{
		TSafeSharedPtr<FNode> FoundNode = FindNode(NodeHandle);
		if (FoundNode)
		{
			ApplyFunc(*FoundNode);
			return EGeometryFlowResult::Ok;
		}
		return EGeometryFlowResult::NodeDoesNotExist;
	}

	UE_API EGeometryFlowResult GetInputTypeForNode(FHandle NodeHandle, FString InputName, int32& Type) const;
	UE_API EGeometryFlowResult GetOutputTypeForNode(FHandle NodeHandle, FString OutputName, int32& Type) const;

	UE_API void ConfigureCachingStrategy(ENodeCachingStrategy NewStrategy);
	UE_API EGeometryFlowResult SetNodeCachingStrategy(FHandle NodeHandle, ENodeCachingStrategy Strategy);

	UE_API FString DebugDumpGraph(TFunction<bool(TSafeSharedPtr<FNode>)> IncludeNodeFn) const;

	/**
	* Serialization operator for FGraph.
	*
	* @param Ar Archive to serialize with.
	* @param Graph Graph to serialize.
	* @returns Passing down serializing archive.
	*/
	friend FArchive& operator<<(FArchive& Ar, FGraph& Graph)
	{
		Graph.Serialize(Ar);
		return Ar;
	}

	/** Serialize this graph to an archive. */
	UE_API void Serialize(FArchive& Ar);


protected:

	enum class ENodeAddResult
	{
		Success = 0,
		Failed_HandleExists = 1,
		Failed_UnregisteredType = 2,
	};


	bool CanAddNodeOfType(const FName TypeName)
	{
		const FNodeFactory& NodeFactory = FNodeFactory::GetInstance();

		return NodeFactory.CanMake(TypeName);
	}

	ENodeAddResult AddNodeOfType(const FName TypeName,
		const FHandle Handle,
		const FString& Identifier,
		ENodeCachingStrategy CachingStrategy)
	{
		if (AllNodes.Contains(Handle))
		{
			return ENodeAddResult::Failed_HandleExists;
		}

		const FNodeFactory& NodeFactory = FNodeFactory::GetInstance();

		if (!NodeFactory.CanMake(TypeName))
		{
			return ENodeAddResult::Failed_UnregisteredType;
		}

		FNodeInfo NewNodeInfo;
		NewNodeInfo.Node = TSharedPtr<FNode, ESPMode::ThreadSafe>(NodeFactory.CreateNodeOfType(TypeName).Release());
		NewNodeInfo.NodeTypeName = TypeName;
		NewNodeInfo.Node->SetIdentifier(Identifier);
		NewNodeInfo.CachingStrategy = CachingStrategy;

		AllNodes.Add(Handle, NewNodeInfo);
		AllNodeLocks.Add(Handle, MakeSafeShared<FRWLock>());
		return ENodeAddResult::Success;
	}

	template<typename T>
	EGeometryFlowResult EvaluateResultInternal(
		FHandle Node,
		FString OutputName,
		T& Storage,
		int32 StorageTypeIdentifier,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo,
		bool bTryTakeResult)
	{
		int32 OutputType;
		EGeometryFlowResult ValidOutput = GetOutputTypeForNode(Node, OutputName, OutputType);
		if (ValidOutput != EGeometryFlowResult::Ok)
		{
			return ValidOutput;
		}
		if (OutputType != StorageTypeIdentifier)
		{
			return EGeometryFlowResult::UnmatchedTypes;
		}
		if (bTryTakeResult)
		{
			TSafeSharedPtr<IData> Data = ComputeOutputData(Node, OutputName, EvaluationInfo, true);
			if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
			{
				return EGeometryFlowResult::OperationCancelled;
			}
			Data->GiveTo(Storage, StorageTypeIdentifier);
		}
		else
		{
			TSafeSharedPtr<IData> Data = ComputeOutputData(Node, OutputName, EvaluationInfo, false);
			if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
			{
				return EGeometryFlowResult::OperationCancelled;
			}
			Data->GetDataCopy(Storage, StorageTypeIdentifier);
		}
		return EGeometryFlowResult::Ok;
	}


	friend class FGeometryFlowExecutor;

	int32 NodeCounter = 0;

	ENodeCachingStrategy DefaultCachingStrategy = ENodeCachingStrategy::AlwaysCache;

	struct FNodeInfo
	{
		TSafeSharedPtr<FNode> Node;
		FName  NodeTypeName = FName("UnknowNodeType");
		ENodeCachingStrategy CachingStrategy = ENodeCachingStrategy::Default;
	};

	TMap<FHandle, FNodeInfo> AllNodes;
	TMap<FHandle, TSafeSharedPtr<FRWLock>> AllNodeLocks;

	UE_API TSafeSharedPtr<FNode> FindNode(FHandle Handle) const;
	
	UE_API ENodeCachingStrategy GetCachingStrategyForNode(FHandle NodeHandle) const;
	UE_API TSafeSharedPtr<FRWLock> FindNodeLock(FHandle Handle) const;

	TArray<FConnection> Connections;

	UE_API EGeometryFlowResult FindConnectionForInput(FHandle ToNode, FString ToInput, FConnection& ConnectionOut) const;

	UE_API int32 CountOutputConnections(FHandle FromNode, const FString& FromOutput) const;

	UE_API TSafeSharedPtr<IData> ComputeOutputData(
		FHandle Node, 
		FString OutputName, 
		TUniquePtr<FEvaluationInfo>& EvaluationInfo,
		bool bStealOutputData = false);

	// retrun @true if all the graph connections needed for this computation exist
	UE_API bool CanComputeOutput(
		FHandle Node,
		FString OutputName);

	/**
	* Visits upstream dependencies for the specified node and calls the Visitor functor (in parallel) on each.
	* Terminates if Vistor returns true, or if the upstream graph is exhausted.
	* @return true if terminated by the Visitor.
	*/
	UE_API bool VistDependencies(FHandle NodeHandle, TFunctionRef<bool(FGraph::FHandle)> Vistor) const;

	bool IsInCycle(FHandle NodeHandle)
	{
		FHandle Start = NodeHandle;
		auto Visitor = [this, Start](FGraph::FHandle UpstreamHandle) {return Start == UpstreamHandle; };
		return VistDependencies(NodeHandle, Visitor);
	}

	FCriticalSection EvaluateLock;
};


inline uint32 GetTypeHash(FGraph::FHandle Handle)
{
	return ::GetTypeHash(Handle.Identifier);
}









}	// end namespace GeometryFlow
}	//

#undef UE_API
