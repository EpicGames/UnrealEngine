// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowGraph.h"
#include "GeometryFlowTSourceNode.h"

namespace UE::GeometryFlow { template <typename T, int32 StorageTypeIdentifier> class TSourceNode; }

namespace UE
{
namespace GeometryFlow
{



template<typename SourceNodeType>
void UpdateSourceNodeValue(FGraph& Graph, FGraph::FHandle NodeHandle, const typename SourceNodeType::CppType& NewValue)
{
	Graph.ApplyToNodeOfType<SourceNodeType>(NodeHandle, [&](SourceNodeType& Node)
	{
		Node.UpdateSourceValue(NewValue);
	});
}


/**
 * Update the integer index that controls which Input to a TSwitchNode will be provided as it's Output.
 */
template<typename SwitchNodeType>
void UpdateSwitchNodeInputIndex(FGraph& Graph, FGraph::FHandle NodeHandle, int32 NewSwitchIndex)
{
	Graph.ApplyToNodeOfType<SwitchNodeType>(NodeHandle, [&](SwitchNodeType& Node)
	{
		Node.UpdateSwitchInputIndex(NewSwitchIndex);
	});
}



template<typename SettingsType>
void UpdateSettingsSourceNodeValue(FGraph& Graph, FGraph::FHandle NodeHandle, const SettingsType& NewSettings)
{
	using SettingsSourceNodeType = TSourceNodeBase<SettingsType, SettingsType::DataTypeIdentifier>;

	Graph.ApplyToNodeOfType<SettingsSourceNodeType>(NodeHandle, [&](SettingsSourceNodeType& Node)
	{
		Node.UpdateSourceValue(NewSettings);
	});
}

// Returns an index into the Connections array or -1 if not found
inline int FindAnyConnectionFromNode(FGraph::FHandle FromNode, const TArray<FGraph::FConnection>& Connections)
{
	for (int ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ++ConnectionIndex)
	{
		if (Connections[ConnectionIndex].FromNode == FromNode)
		{
			return ConnectionIndex;
		}
	}

	return -1;
}

// Returns an index into the Connections array or -1 if not found
inline int FindAnyConnectionToNode(FGraph::FHandle ToNode, const TArray<FGraph::FConnection>& Connections)
{
	for (int ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ++ConnectionIndex)
	{
		if (Connections[ConnectionIndex].ToNode == ToNode)
		{
			return ConnectionIndex;
		}
	}

	return -1;
}

// Returns an indices into the Connections array
inline TArray<int> FindAllConnectionsToNode(FGraph::FHandle ToNodeID, const TArray<FGraph::FConnection>& Connections)
{
	TArray<int> Found;
	for (int ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ++ConnectionIndex)
	{
		if (Connections[ConnectionIndex].ToNode == ToNodeID)
		{
			Found.Add(ConnectionIndex);
		}
	}

	return Found;
}



}	// end namespace GeometryFlow
}	// end namespace UE
