// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"


namespace UE
{
namespace GeometryFlow
{




template<typename T, int32 DataTypeIdentifier>
class TImmutableData : public IData
{
public:
	TUniquePtr<const T> Data;

	TImmutableData()
	{
	}

	TImmutableData(TUniquePtr<const T>&& DataIn)
	{
		Data = MoveTemp(DataIn);
	}

	virtual int32 GetPayloadType() const override
	{
		return DataTypeIdentifier;
	}

	virtual int64 GetPayloadBytes() const override
	{
		check(false);
		return 0;
	}

	virtual bool CopyData(void* StorageType, int32 AsType) const override
	{
		check(false);
		return false;
	}

	virtual bool MoveDataOut(void* StorageType, int32 AsType) override
	{
		check(false);
		return false;
	}

protected:
	
	// This is used by IData::GetDataConstRef(). Should probably not be
	// called in other contexts...
	virtual const void* GetRawDataPointerUnsafe() const override
	{
		return (const void *)Data.Get();
	}


	const T& GetData() const
	{
		return *Data;
	}
};





template<typename T, int32 StorageTypeIdentifier>
class TImmutableNodeInput : public INodeInput
{
public:

	virtual int32 GetDataType() const { return StorageTypeIdentifier; }


public:
	virtual bool CanTransformInput() const override 
	{ 
		ensure(InputFlags.bCanTransformInput == false); 
		return false;
	}
};




template<typename T, int32 StorageTypeIdentifier>
class TImmutableNodeOutput final : public INodeOutput
{
public:
	TImmutableNodeOutput()
	{
		TUniquePtr<const T> InitialValue = MakeUnique<const T>();
		UpdateOutput(MakeShared<TImmutableData<T, StorageTypeIdentifier>, ESPMode::ThreadSafe>(MoveTemp(InitialValue)));
	}

	virtual int32 GetDataType() const { return StorageTypeIdentifier; }
};




}	// end namespace GeometryFlow
}	// end namespace UE
