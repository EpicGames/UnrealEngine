// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "Serialization/Archive.h"


namespace UE
{
namespace GeometryFlow
{



// This must be specialized for new data types.
// see GeometryFlowTSourceNode.h for example
template <typename DataType>
struct TSerializationMethod
{
	static void Serialize(FArchive& Ar, DataType& T);
};



template<typename T, int32 DataTypeIdentifier>
class TMovableData : public IData
{
	typedef IData Super;
public:
	T Data;

	TMovableData()
	{
	}

	TMovableData(const T& InitialValue)
	{
		Data = InitialValue;
	}

	virtual int32 GetPayloadType() const override
	{
		return DataTypeIdentifier;
	}

	virtual int64 GetPayloadBytes() const override
	{
		return sizeof(Data);
	}

	virtual bool CopyData(void* StorageType, int32 AsType) const override
	{
		if (ensure(AsType == GetPayloadType()))
		{
			*((T*)StorageType) = Data;
			return true;
		}
		return false;
	}

	virtual bool MoveDataOut(void* StorageType, int32 AsType) override
	{
		if (ensure(AsType == GetPayloadType()))
		{
			*((T*)StorageType) = MoveTemp(Data);
			return true;
		}
		return false;
	}

protected:
	virtual const void* GetRawDataPointerUnsafe() const override
	{
		return (const void *)&Data;
	}

public:


	void GetData(T& DataOut) const
	{
		DataOut = Data;
	}

	void SetData(const T& DataIn)
	{
		Data = DataIn;
		IncrementTimestamp();
	}

	void MoveData(T&& DataIn)
	{
		Data = MoveTemp(DataIn);
		IncrementTimestamp();
	}

	void Serialize(FArchive& Ar)
	{
		Ar << IData::Timestamp;
		TSerializationMethod<T>::Serialize(Ar, Data);
	}
};





template<typename T, int32 StorageTypeIdentifier>
class TBasicNodeInput : public INodeInput
{
public:

	virtual int32 GetDataType() const { return StorageTypeIdentifier; }
};




template<typename T, int32 StorageTypeIdentifier>
class TBasicNodeOutput final : public INodeOutput
{
public:
	TBasicNodeOutput()
	{
		T InitialValue;
		UpdateOutput(MakeShared<TMovableData<T, StorageTypeIdentifier>, ESPMode::ThreadSafe>(InitialValue));
	}

	virtual int32 GetDataType() const { return StorageTypeIdentifier; }
};




}	// end namespace GeometryFlow
}	// end namespace UE
