// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowTypes.h"
#include "UObject/StructOnScope.h"
#include "Serialization/Archive.h"

#define UE_API GEOMETRYFLOWCORE_API

namespace UE::GeometryFlow { class FNode; }


// TODO
// - handling for array types
// - cancellation support
// - can we have the concept of data shared between nodes? ie when recomputing normals, we currently 'steal'
//   but really we are only partially modifying input data...   (maybe cleaner to just separate out compute and set normals?)

class FProgressCancel;

namespace UE
{
namespace GeometryFlow
{



class IData
{
public:
	virtual ~IData() = default;

	virtual int32 GetPayloadType() const = 0;
	virtual int64 GetPayloadBytes() const = 0;
	virtual bool CopyData(void* StorageType, int32 AsType) const = 0;
	virtual bool MoveDataOut(void* StorageType, int32 AsType) = 0;

protected:
	virtual const void* GetRawDataPointerUnsafe() const = 0;

public:

	virtual bool IsSameType(const IData& OtherType)
	{
		return GetPayloadType() == OtherType.GetPayloadType();
	}

	template<typename T>
	void GetDataCopy(T& DataOut, int32 AsType) const
	{
		CopyData((void*)&DataOut, AsType);
	}

	template<typename T>
	const T& GetDataConstRef(int32 AsType) const
	{
		check(GetPayloadType() == AsType);
		return *(const T*)GetRawDataPointerUnsafe();
	}

	template<typename T>
	void GiveTo(T& DataOut, int32 AsType)
	{
		MoveDataOut((void*)&DataOut, AsType);
	}

public:
	virtual int32 GetTimestamp() const
	{
		return Timestamp;
	}

protected:
	int32 Timestamp = 0;

	friend class INodeOutput;

	virtual void IncrementTimestamp()
	{
		Timestamp++;
	}

	virtual void SetMaxTimestamp(int32 ExternalTimestamp)
	{
		Timestamp = FMath::Max(ExternalTimestamp, Timestamp);
	}

};



struct FNodeInputFlags
{
	bool bCanTransformInput = false;

	static FNodeInputFlags Transformable() { return FNodeInputFlags{ true }; }
};


class INodeInput
{
public:
	virtual ~INodeInput() = default;

	/*
	 * INodeInput virtual API that implementations must implement
	 */

	virtual int32 GetDataType() const = 0;



	/*
	 * INodeInput control API
	 */

public:
	virtual FNodeInputFlags GetInputFlags() const { return InputFlags; }
	virtual bool CanTransformInput() const { return InputFlags.bCanTransformInput; }
protected:
	FNodeInputFlags InputFlags;
	virtual void SetInputFlags(FNodeInputFlags SetFlags) { InputFlags = SetFlags; }

	friend class FNode;
};





class INodeOutput
{
public:
	virtual ~INodeOutput() = default;

	virtual int32 GetDataType() const = 0;

	virtual void UpdateOutput(TSafeSharedPtr<IData> NewData)
	{
		if (ensure(NewData->GetPayloadType() == GetDataType()))
		{
			CachedValue = NewData;
			CachedValue->SetMaxTimestamp(LastDataTimestamp + 1);
			LastDataTimestamp = CachedValue->GetTimestamp();
		}
	}

	virtual bool HasCachedOutput() const
	{
		return (CachedValue != nullptr);
	}

	virtual void ClearOutputCache()
	{
		CachedValue = TSafeSharedPtr<IData>();
	}

	virtual TSafeSharedPtr<IData> GetOutput() const
	{
		check(CachedValue != nullptr);
		return CachedValue;
	}

	virtual TSafeSharedPtr<IData> StealOutput()
	{
		check(CachedValue != nullptr);
		TSafeSharedPtr<IData> Result = CachedValue;
		CachedValue = TSafeSharedPtr<IData>();
		LastDataTimestamp++;
		return Result;
	}

protected:
	int32 LastDataTimestamp = 0;
	TSafeSharedPtr<IData> CachedValue;
};



struct FDataFlags
{
	bool bIsMutableData = false;
};


class FNamedDataMap
{
protected:
	TArray<FString> Names;
	TArray<TSafeSharedPtr<IData>> Datas;
	TArray<FDataFlags> DatasFlags;

public:
	void Add(const FString& Name, FDataFlags Flags = FDataFlags())
	{
		check(Contains(Name) == false);
		Names.Add(Name);
		Datas.Add(TSafeSharedPtr<IData>());
		DatasFlags.Add(Flags);
	}

	void Add(const FString& Name, TSafeSharedPtr<IData> Data, FDataFlags Flags = FDataFlags())
	{
		check(Contains(Name) == false);
		Names.Add(Name);
		Datas.Add(Data);
		DatasFlags.Add(Flags);
	}

	bool Contains(const FString& Name) const { return Names.Contains(Name); }

	const TArray<FString>& GetNames() const { return Names; }

	TSafeSharedPtr<IData> FindData(const FString& Name) const
	{
		int32 Index = Names.IndexOfByKey(Name);
		if (ensure(Index != INDEX_NONE))
		{
			return Datas[Index];
		}
		return nullptr;
	}

	FDataFlags GetDataFlags(const FString& Name) const
	{
		int32 Index = Names.IndexOfByKey(Name);
		if (ensure(Index != INDEX_NONE))
		{
			return DatasFlags[Index];
		}
		return FDataFlags();
	}

	bool SetData(const FString& Name, TSafeSharedPtr<IData> Data)
	{
		int32 Index = Names.IndexOfByKey(Name);
		if (ensure(Index != INDEX_NONE))
		{
			Datas[Index] = Data;
			return true;
		}
		return false;
	}
};




class FEvaluationInfo
{
public:
	UE_API FEvaluationInfo();
	virtual ~FEvaluationInfo() = default;

	UE_API virtual void Reset();

	UE_API virtual void CountEvaluation(FNode* Node);
	int NumEvaluations() const { return EvaluationsCount; }

	UE_API virtual void CountCompute(FNode* Node);
	int NumComputes() const { return ComputesCount; }

	FProgressCancel* Progress = nullptr;

protected:
	std::atomic<int> EvaluationsCount;
	std::atomic<int> ComputesCount;
};




struct FEvalRequirement
{
	FString InputName;
	FNodeInputFlags InputFlags;

	FEvalRequirement() {}
	FEvalRequirement(const FString& Name) : InputName(Name) {}

	FEvalRequirement(const FString& NameIn, FNodeInputFlags InputFlagsIn)
		: InputName(NameIn), InputFlags(InputFlagsIn) {}
};



class FNode
{
public:

	FNode() = default;
	FNode(const FNode&) = delete;
	FNode& operator=(const FNode&) = delete;
	FNode(FNode&&) = default;
	FNode& operator=(FNode&&) = default;
	virtual ~FNode() = default;

	//--- Derived nodes should implement these quasi-RTTI methods
	//    using the provided macro GEOMETRYFLOW_NODE_RTTI()
	static FName StaticType() { return "FNode"; }
	virtual FName GetType() const { return "FNode"; }
	virtual bool IsA(FName InType) const
	{
		return InType.ToString().Equals(StaticType().ToString());
	}
	//---
	
	void SetIdentifier(const FString& IdentifierIn) { Identifier = IdentifierIn; }
	const FString& GetIdentifier() const { return Identifier; }

	// version number used for serialization - to allow for changes in node data.
	virtual int32 GetVersionID() const { return -1; }
	virtual void Serialize(FArchive& Ar)
	{
		if (Ar.IsLoading())
		{
			int32 SavedVersion;
			Ar << SavedVersion;

			const int32 ExpectedVersion = GetVersionID();
			if (SavedVersion != ExpectedVersion)
			{
				UpdateVersion(Ar, SavedVersion);
			}
		}
		else
		{
			int32 ThisVersion = GetVersionID();
			Ar << ThisVersion;
		}
	}

	UE_API EGeometryFlowResult GetInputType(FString Name, int32& TypeOut) const;
	UE_API EGeometryFlowResult GetOutputType(FString Name, int32& TypeOut) const;

	UE_API FNodeInputFlags GetInputFlags(FString InputName) const;

	UE_API virtual void EnumerateInputs(TFunctionRef<void(const FString& Name, const TUniquePtr<INodeInput>& Input)> EnumerateFunc) const;
	UE_API virtual void EnumerateOutputs(TFunctionRef<void(const FString& Name, const TUniquePtr<INodeOutput>& Output)> EnumerateFunc) const;

	UE_API virtual bool IsOutputAvailable(FString OutputName) const;

	UE_API virtual TSafeSharedPtr<IData> StealOutput(FString OutputName);

	UE_API TSafeSharedPtr<IData> GetOutput(const FString& OutputName) const;

	/**
	 * Find the list of named inputs that must be available to compute the named Outputs.
	 * By default will return all Inputs if any of the listed Outputs exists on this Node.
	 */
	UE_API virtual void CollectRequirements(
		const TArray<FString>& Outputs, 
		TArray<FEvalRequirement>& RequiredInputsOut);

	UE_API virtual void CollectAllRequirements(TArray<FEvalRequirement>& RequiredInputsOut);

	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) = 0;

	virtual TSafeSharedPtr<IData> GetDefaultInputData(FString InputName) const
	{
		if (InputDefaultValues.Contains(InputName))
		{
			return InputDefaultValues[InputName];
		}
		return TSafeSharedPtr<IData>();
	}

protected:

	// derived types configure inputs and outputs in their constructor
	UE_API void AddInput(FString Name, TUniquePtr<INodeInput>&& Input, TSafeSharedPtr<IData> DefaultData = TSafeSharedPtr<IData>());
	UE_API void AddOutput(FString Name, TUniquePtr<INodeOutput>&& Output);


	friend class FGraph;
	friend class FGeometryFlowExecutor;

	FString Identifier;

	struct FNodeInputInfo
	{
		// explicitly non-copyable because of embedded TUniquePtr
		FNodeInputInfo() = default;
		FNodeInputInfo(const FNodeInputInfo&) = delete;
		FNodeInputInfo& operator=(const FNodeInputInfo&) = delete;
		FNodeInputInfo(FNodeInputInfo&&) = default;
		FNodeInputInfo& operator=(FNodeInputInfo&&) = default;

		FString Name;
		int32 LastTimestamp = -1;
		TUniquePtr<INodeInput> Input;
	};

	struct FNodeOutputInfo
	{
		// explicitly non-copyable because of embedded TUniquePtr
		FNodeOutputInfo() = default;
		FNodeOutputInfo(const FNodeOutputInfo&) = delete;
		FNodeOutputInfo& operator=(const FNodeOutputInfo&) = delete;
		FNodeOutputInfo(FNodeOutputInfo&&) = default;
		FNodeOutputInfo& operator=(FNodeOutputInfo&&) = default;

		FString Name;
		TUniquePtr<INodeOutput> Output;
	};

	UE_API virtual void SetOutput(
		const FString& OutputName,
		TSafeSharedPtr<IData> NewData
	);

	UE_API virtual void ClearOutput(const FString& OutputName);
	UE_API virtual void ClearAllOutputs();

	TArray<FNodeInputInfo> NodeInputs;
	TArray<FNodeOutputInfo> NodeOutputs;

	TMap<FString, TSafeSharedPtr<IData>> InputDefaultValues;

protected:
	// helper functions for setup
	UE_API void ConfigureInputFlags(const FString& InputName, FNodeInputFlags Flags);

	// helper functions for evaluation
	UE_API bool IsInputDirty(FString Name, int32 NewTimestamp) const;
	UE_API bool CheckIsInputDirtyAndUpdate(FString Name, int32 NewTimestamp);
	UE_API void UpdateInputTimestamp(FString Name, int32 NewTimestamp);

	UE_API TSafeSharedPtr<IData> FindAndUpdateInputForEvaluate(const FString& InputName, const FNamedDataMap& DatasIn,
		bool& bAccumModifiedOut, bool& bAccumValidOut);

protected:

	// override this in derived node if the version number has been bumped
	virtual void UpdateVersion(FArchive& Ar, int32 SavedVersion) {}
};

template<typename NodeType>
NodeType* CastToNodePtr(FNode* Node)
{
	NodeType* Result = nullptr;
	if (Node->IsA(NodeType::StaticType()))
	{
		Result = static_cast<NodeType*>(Node);
	}
	return Result;
}

#define GEOMETRYFLOW_NODE_RTTI(TYPE, SuperType)                                             \
public:                                                                                     \
	typedef SuperType   Super;                                                              \
	static FName StaticType() { return #TYPE;}                                              \
	virtual FName GetType() const override {return #TYPE;}                                  \
	virtual bool  IsA(FName InType) const override                                          \
		{ return InType.ToString().Equals(StaticType().ToString()) || Super::IsA(InType); }	\
private:                                                      	

#define GEOMETRYFLOW_NODE_INTERNAL(TYPE, VersionID, SuperType)            \
public:                                                                   \
	virtual int32 GetVersionID() const override { return VersionID;}      \
    GEOMETRYFLOW_NODE_RTTI(TYPE, SuperType)                               \



}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
