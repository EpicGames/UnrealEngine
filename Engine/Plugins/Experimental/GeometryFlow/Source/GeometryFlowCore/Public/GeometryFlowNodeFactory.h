// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "CoreMinimal.h"


class FLazySingleton;

namespace UE
{
namespace GeometryFlow
{

	// Singleton used to create nodes UE::GeometryFlow::FGraph, introduced to help with serialization.  
	// this contains an map between FNames and function pointers that can create the named nodes.
	class  FNodeFactory
	{

	public:

		struct FNodeFactoryInfo
		{
			typedef TFunction<TUniquePtr<FNode>() > FNodeTypeFactory;
			
			FString TypePrettyName;
			FString TypeCategory;
			FNodeTypeFactory  TypeFactory;
		};


		FNodeFactory(const FNodeFactory&) = delete;
	
	

		GEOMETRYFLOWCORE_API static  FNodeFactory& GetInstance();
		GEOMETRYFLOWCORE_API static  void TearDown();

		// returns false if the NodeTypeName is already registered.
		template <typename NodeType>
		bool RegisterType(FString NodeTypePrettyName = FString(), FString TypeCategory = FString())
		{
			const FName NodeTypeName = NodeType::StaticType();
			if (Factories.Contains(NodeTypeName))
			{
				return false;
			}

			if (NodeTypePrettyName.IsEmpty())
			{
				NodeTypePrettyName = NodeTypeName.ToString();
			}
			if (TypeCategory.IsEmpty())
			{
				TypeCategory = FString("Geometry Flow");
			}

			FNodeFactoryInfo NodeFactoryInfo= {NodeTypePrettyName, TypeCategory, []() { return MakeUnique<NodeType>(); } };

			Factories.Add(NodeTypeName, NodeFactoryInfo);
			return true;
		}

		template <typename NodeType>
		bool CanMake() const
		{
			return CanMake(NodeType::StaticType());
		}


		bool CanMake(const FName NodeTypeName) const
		{
			return Factories.Contains(NodeTypeName);
		}

		// For unregistered types, this returns a null ptr.
		TUniquePtr<FNode> CreateNodeOfType(const FName NodeTypeName) const 
		{
			TUniquePtr<FNode> Result(nullptr);
			if (CanMake(NodeTypeName))
			{
				const FNodeFactoryInfo& FactoryInfo = Factories[NodeTypeName];
				Result = FactoryInfo.TypeFactory();
			}
			return Result;
		}

		const TMap<const FName, FNodeFactory::FNodeFactoryInfo>& GetFactories() const { return Factories;}

	protected:

		TMap<const FName, FNodeFactoryInfo>  Factories;
		
		
		friend FLazySingleton; // needed because the FNodeFactory constructor is protected.
		FNodeFactory()
		{}

	
	};

	

}



// by convention we register the nodes with an FName derived from the actual node class name
#define GEOMETRYFLOW_QUOTE(x) #x

// Register type F{ReadableName} with the FName {ReadableName}
#define GEOMETRYFLOW_REGISTER_NODE_TYPE(ReadableName)                                                           \
	FNodeFactory::GetInstance().RegisterType<F##ReadableName##Node>( FString(GEOMETRYFLOW_QUOTE(ReadableName)), \
																	 CategoryName); 

// Register Source Node type F{ReadableName}SourceNode  with FName {ReadableName}
#define GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(ReadableName)                                                         \
	FNodeFactory::GetInstance().RegisterType<F##ReadableName##SourceNode>( FString(GEOMETRYFLOW_QUOTE(ReadableName)),\
																		   CategoryName);

// Register Setting Source Node type  F{ReadableName}SettingsSourceNode with FName {ReadableName}Settings
#define GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(ReadableName)                                                                         \
	FNodeFactory::GetInstance().RegisterType<F##ReadableName##SettingsSourceNode>( FString(GEOMETRYFLOW_QUOTE(ReadableName##Settings)),\
																				   CategoryName);

// Register two nodes (an opperator and its settings).  F{ReadableName}Node with FName {ReadableName} and  F{ReadableName}SettingsSourceNode with FName {ReadableName}Settings
#define GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(ReadableName) \
	FNodeFactory::GetInstance().RegisterType<F##ReadableName##Node>(FString(GEOMETRYFLOW_QUOTE(ReadableName)), \
																	CategoryName);                             \
																	                                           \
	FNodeFactory::GetInstance().RegisterType<F##ReadableName##SettingsSourceNode>( FString(GEOMETRYFLOW_QUOTE(ReadableName##Settings)),\
																				   CategoryName);

}
