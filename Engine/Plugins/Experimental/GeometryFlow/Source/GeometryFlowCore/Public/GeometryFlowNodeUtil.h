// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNode.h"
#include "GeometryFlowMovableData.h"


namespace UE
{
namespace GeometryFlow
{



template<typename DataType>
TSafeSharedPtr<TMovableData<DataType, DataType::DataTypeIdentifier>> MakeMovableData(DataType&& Data)
{
	return MakeShared<TMovableData<DataType, DataType::DataTypeIdentifier>, ESPMode::ThreadSafe>(MoveTemp(Data));
}


template<typename DataType>
TUniquePtr<TBasicNodeInput<DataType, DataType::DataTypeIdentifier>> MakeBasicInput()
{
	return MakeUnique<TBasicNodeInput<DataType, DataType::DataTypeIdentifier>>();
}

template<typename DataType>
TUniquePtr<TBasicNodeOutput<DataType, DataType::DataTypeIdentifier>> MakeBasicOutput()
{
	return MakeUnique<TBasicNodeOutput<DataType, DataType::DataTypeIdentifier>>();
}

template<typename T>
EGeometryFlowResult ExtractData(TSafeSharedPtr<IData> Data,
								T& Storage,
								int32 StorageTypeIdentifier,
								bool bTryTakeResult)
{
	if (Data->GetPayloadType() != StorageTypeIdentifier)
	{
		return EGeometryFlowResult::UnmatchedTypes;
	}
	if (bTryTakeResult)
	{
		Data->GiveTo(Storage, StorageTypeIdentifier);
	}
	else
	{
		Data->GetDataCopy(Storage, StorageTypeIdentifier);
	}

	return EGeometryFlowResult::Ok;

}



}	// end namespace GeometryFlow
}	// end n
