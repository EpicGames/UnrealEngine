// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryBase.h"
#include "GeometryFlowNode.h"
#include "GeometryFlowMovableData.h"
#include "Templates/UniquePtr.h"


namespace UE::GeometryFlow { template <typename T, int32 StorageTypeIdentifier> class TBasicNodeInput; }
namespace UE::GeometryFlow { template <typename T, int32 StorageTypeIdentifier> class TBasicNodeOutput; }

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

class FSourceNodeBase : public FNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FSourceNodeBase, Version, FNode)
public:
	FSourceNodeBase() = default;

	virtual ~FSourceNodeBase() = default;


	virtual int GetSourceDataType() const {return -1;}

	// source nodes have no input ( hence no requirements)
	virtual void CollectRequirements(const TArray<FString>& Outputs, TArray<FEvalRequirement>& RequiredInputsOut) override
	{
		return;
	}
	// source nodes have no input ( hence no requirements)
	virtual void CollectAllRequirements(TArray<FEvalRequirement>& RequiredInputsOut) override
	{
		return;
	}


	// copy the source data into an object that can be displayed in UI
	virtual TSharedPtr<FStructOnScope> AsStructOnScope() const = 0;

	// allow the source data to be updated (to reflect changes in the UI)
	virtual void UpdateSourceFromStructOnScope(const FStructOnScope& StructOnScope) = 0;

};

template<typename T, int32 StorageTypeIdentifier>
class  TSourceNodeBase : public FSourceNodeBase
{
protected:
	using DataType = TMovableData<T, StorageTypeIdentifier>;

	TSafeSharedPtr<DataType> Value;

public:
	using CppType = T;
	static constexpr int DataTypeIdentifier = StorageTypeIdentifier;

	static const FString OutParamValue() { return TEXT("Value"); }

public:
	TSourceNodeBase()
	{
		T InitialValue;
		Value = MakeSafeShared<DataType>(InitialValue);
		AddOutput(OutParamValue(), MakeUnique<TBasicNodeOutput<T, StorageTypeIdentifier>>());

		UpdateSourceValue(InitialValue);
	}

	virtual ~TSourceNodeBase() = default;

	virtual int GetSourceDataType() const override { return DataTypeIdentifier; }

	void UpdateSourceValue(const T& NewValue)
	{
		Value->SetData(NewValue);
		SetOutput(OutParamValue(), Value);
	}
	
	void GetSourceValue(T& OutValue)
	{
		Value->GetData(OutValue);
	}

	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamValue())))
		{
			DatasOut.SetData(OutParamValue(), GetOutput(OutParamValue()));
		}
	}

	// copy the UStruct into an object that can be displayed
	virtual TSharedPtr<FStructOnScope> AsStructOnScope() const override
	{
		TSharedPtr<FStructOnScope> StructOnScope(nullptr);
		return StructOnScope;
	}

	// used to update the source UStruct when the displayed version has been changed.
	virtual void UpdateSourceFromStructOnScope(const FStructOnScope& StructOnScope) override
	{
	}

	virtual void Serialize(FArchive& Ar) override
	{
		Value->Serialize(Ar);
	}
};


// Add ability to expose parameters to UI.
// Note This can only be made for source nodes that have the correct UStructType.
template<typename UStructType, int32 StorageTypeIdentifier>
class TUStructSourceNode : public TSourceNodeBase<UStructType, StorageTypeIdentifier>
{
	using TSourceNodeBase<UStructType, StorageTypeIdentifier>::Value;

public:
	using CppType = UStructType;


public:

	virtual ~TUStructSourceNode() = default;

	// copy the UStruct into an object that can be displayed
	virtual TSharedPtr<FStructOnScope> AsStructOnScope() const override
	{
		typedef TStructOnScope<UStructType>  FTypedStructOnScope;

		TSharedPtr<FTypedStructOnScope> StructOnScope = MakeShared< FTypedStructOnScope >();
		StructOnScope->template InitializeAs<UStructType>(Value.Get()->template GetDataConstRef<UStructType>(StorageTypeIdentifier));

		return StructOnScope;
	}

	// used to update the source UStruct when the displayed version has been changed.
	virtual void UpdateSourceFromStructOnScope(const FStructOnScope& StructOnScope) override
	{
		TStructOnScope<UStructType> TypedStructOnScope;
		TypedStructOnScope.InitializeFromChecked(StructOnScope);

		UStructType TmpParameters;
		TmpParameters = *TypedStructOnScope.Get();
		this->UpdateSourceValue(TmpParameters);
	}


};

namespace
{

	template <typename PODType>
	void PODSerializer(FArchive& Ar, PODType& Data)
	{
		Ar << Data;
	}

	template <typename PODStructType>
	void PODStructSerializer(FArchive& Ar, PODStructType& Data)
	{
		int64 NumBits;
		if (Ar.IsSaving())
		{
			NumBits = sizeof(Data)  * 8;
		}
		Ar << NumBits;
		Ar.SerializeBits((void*)&Data, NumBits);
	}

	template <typename UStructType>
	void UStructSerializer(FArchive& Ar, UStructType& Data)
	{
		UScriptStruct* const Struct = UStructType::StaticStruct(); // singleton
		Struct->SerializeTaggedProperties(Ar, (uint8*)&Data, Struct, nullptr);
	}
}

/**
* This macro declares a MovableData, Node Input/Output, and TSourceNode type for a given C++ type/identifier
*
* eg GEOMETRYFLOW_DECLARE_BASIC_TYPES(Int32, int, (int)EDataTypes::Integer) will declare/define:
*     - TMovableData type named FDataInt32
*     - TBasicNodeInput type named FInt32Input
*     - TBasicNodeOutput type named FInt32Output
*     - TSourceNode type named FInt32SourceNode
*
*/
#define GEOMETRYFLOW_DECLARE_BASIC_TYPES_WO_SERIALIZATION(TypeName, CppType, TypeIdentifier, VersionIntID)      \
typedef TMovableData<CppType, TypeIdentifier> FData##TypeName;                   \
typedef TBasicNodeInput<CppType, TypeIdentifier> F##TypeName##Input;             \
typedef TBasicNodeOutput<CppType, TypeIdentifier> F##TypeName##Output;           \
class F##TypeName##SourceNode : public TSourceNodeBase<CppType, TypeIdentifier>  { static constexpr int Version = VersionIntID;  GEOMETRYFLOW_NODE_INTERNAL(F##TypeName##SourceNode, Version, FSourceNodeBase) }; 

/**
* This macro declares a MovableData, Node Input/Output, and TSourceNode type for a given C++ type/identifier
* in addition to simple POD serialization for the movable data type
* 
* eg GEOMETRYFLOW_DECLARE_BASIC_TYPES(Int32, int, (int)EDataTypes::Integer) will declare/define:
*     - TMovableData type named FDataInt32
*     - TBasicNodeInput type named FInt32Input
*     - TBasicNodeOutput type named FInt32Output
*     - TSourceNode type named FInt32SourceNode
*
*/
#define GEOMETRYFLOW_DECLARE_BASIC_TYPES(TypeName, CppType, TypeIdentifier, VersionIntID)      \
template<> struct TSerializationMethod<CppType> { static void Serialize(FArchive& Ar, CppType& Data){ PODStructSerializer(Ar, Data); } }; \
GEOMETRYFLOW_DECLARE_BASIC_TYPES_WO_SERIALIZATION(TypeName, CppType, TypeIdentifier, VersionIntID) \

/**
* This macro declares a MovableData, Node Input/Output, and TSourceNode type for a given C++ type/identifier
* and injects null serialization code.
*
* eg GEOMETRYFLOW_DECLARE_BASIC_TYPES(Int32, int, (int)EDataTypes::Integer) will declare/define:
*     - TMovableData type named FDataInt32
*     - TBasicNodeInput type named FInt32Input
*     - TBasicNodeOutput type named FInt32Output
*     - TSourceNode type named FInt32SourceNode
*
*/
#define GEOMETRYFLOW_DECLARE_BASIC_TYPES_NULL_SERIALIZE(TypeName, CppType, TypeIdentifier, VersionIntID)      \
template<> struct TSerializationMethod<CppType> { static void Serialize(FArchive& Ar, CppType& Data){} }; \
GEOMETRYFLOW_DECLARE_BASIC_TYPES_WO_SERIALIZATION(TypeName, CppType, TypeIdentifier, VersionIntID) \

/**
* This macro declares a MovableData, Node Input/Output, and TSourceNode type for a given C++ type/identifier
*
* eg GEOMETRYFLOW_DECLARE_BASIC_TYPES(Int32, FInt32Setting, (int)EDataTypes::Integer) will declare/define:
*     - TMovableData type named FDataInt32
*     - TBasicNodeInput type named FInt32Input
*     - TBasicNodeOutput type named FInt32Output
*     - TSourceNode type named FInt32SourceNode
*
* where FInt32Setting is a UStruct
*/
#define GEOMETRYFLOW_DECLARE_BASIC_USTRUCT_TYPES(TypeName, CppType, TypeIdentifier, VersionIntID) \
template<> struct TSerializationMethod<CppType> { static void Serialize(FArchive& Ar, CppType& Data){ UStructSerializer(Ar, Data); } }; \
typedef TMovableData<CppType, TypeIdentifier> FData##TypeName##Struct;              \
typedef TBasicNodeInput<CppType, TypeIdentifier> F##TypeName##Struct##Input;                \
typedef TBasicNodeOutput<CppType, TypeIdentifier> F##TypeName##Struct##Output;              \
class F##TypeName##Struct##SourceNode : public TUStructSourceNode<CppType, TypeIdentifier>  { static constexpr int Version = VersionIntID; GEOMETRYFLOW_NODE_INTERNAL(F##TypeName##Struct##SourceNode, Version, FSourceNodeBase) }; \



/**
* This macro defines two classes for a given C++ class:
*   - TMovableData typedef named FData[XYZ]Settings
*   - TSourceNode typedef named F[XYZ]SettingsSourceNode
*
* The assumption is the class has an integer static/constexpr member SettingsType::DataTypeIdentifier that defines the type integer
*/
#define GEOMETRYFLOW_DECLARE_SETTINGS_TYPES_WO_SERIALIZATION(CppType, ReadableName, VersionIntID) \
typedef TMovableData<CppType, CppType::DataTypeIdentifier> FData##ReadableName##Settings; \
class F##ReadableName##SettingsSourceNode : public TSourceNodeBase<CppType, CppType::DataTypeIdentifier>  { static constexpr int Version = VersionIntID; GEOMETRYFLOW_NODE_INTERNAL(F##ReadableName##SettingsSourceNode, Version, FSourceNodeBase) }; \


/**
* This macro defines two classes for a given C++ class:
*   - TMovableData typedef named FData[XYZ]Settings
*   - TSourceNode typedef named F[XYZ]SettingsSourceNode
*
* The assumption is the class has an integer static/constexpr member SettingsType::DataTypeIdentifier that defines the type integer
*/
#define GEOMETRYFLOW_DECLARE_SETTINGS_TYPES(CppType, ReadableName, VersionIntID) \
template<> struct TSerializationMethod<CppType> { static void Serialize(FArchive& Ar, CppType& Data){ PODSerializer(Ar, Data); } }; \
GEOMETRYFLOW_DECLARE_SETTINGS_TYPES_WO_SERIALIZATION(CppType, ReadableName, VersionIntID)


/**
* This macro defines two classes for a given C++ class:
*   - TMovableData typedef named FData[XYZ]Settings
*   - TSourceNode typedef named F[XYZ]SettingsSourceNode
*
* The assumption is the class has an integer static/constexpr member SettingsType::DataTypeIdentifier that defines the type integer
*/
#define GEOMETRYFLOW_DECLARE_SETTINGS_TYPES_NULL_SERIALIZE(CppType, ReadableName, VersionIntID) \
template<> struct TSerializationMethod<CppType> { static void Serialize(FArchive& Ar, CppType& Data){ } }; \
GEOMETRYFLOW_DECLARE_SETTINGS_TYPES_WO_SERIALIZATION(CppType, ReadableName, VersionIntID)

#define GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(CppType, ReadableName, VersionIntID) \
template<> struct TSerializationMethod<CppType> { static void Serialize(FArchive& Ar, CppType& Data){ UStructSerializer(Ar, Data); } }; \
typedef TMovableData<CppType, CppType::DataTypeIdentifier> FData##ReadableName##Settings; \
class F##ReadableName##SettingsSourceNode : public TUStructSourceNode<CppType, CppType::DataTypeIdentifier>{ static constexpr int Version = VersionIntID; GEOMETRYFLOW_NODE_INTERNAL(F##ReadableName##SettingsSourceNode, Version, FSourceNodeBase) }; \



}	// end namespace GeometryFlow
}
