// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE
{
namespace GeometryFlow
{

// alias for Thread-Safe TSharedPtr
template <typename T> 
using TSafeSharedPtr = TSharedPtr<T, ESPMode::ThreadSafe>;

// wrapper around MakeShared for creating a TSafeSharedPtr
template<typename T, typename... InArgTypes>
TSafeSharedPtr<T> MakeSafeShared(InArgTypes&&... Args)
{
	return MakeShared<T, ESPMode::ThreadSafe>(Forward<InArgTypes>(Args)...);
}


// predeclare core types
class IData;
class INodeInput;
class INodeOutput;
class FNode;
class FGraph;



enum class EGeometryFlowResult
{
	Ok = 0,
	NodeDoesNotExist = 1,
	InputDoesNotExist = 2,
	OutputDoesNotExist = 3,
	UnmatchedTypes = 4,
	ConnectionDoesNotExist = 5,
	NoMatchesFound = 6,
	MultipleMatchingAmbiguityFound = 7,
	OperationCancelled = 8,
	InputAlreadyConnected = 9,
	ConnectionRejectedCreatesCycle = 10
};


/**
 * Every data type in a GeometryFlow graph requires a unique integer to identify the type.
 * This enum defines various integer ranges used by the known data/node libraries.
 */
enum class EDataTypes
{
	Integer = 0,
	Float = 1,
	Double = 2,
	
	IntegerStruct = 3,
	FloatStruct = 4,
	DoubleStruct = 5,

	Vector3f = 20,
	Vector3d = 21,

	Vector3fStruct = 22,
	Vector3dStruct = 23,

	Name = 30,

	NameStruct = 31,

	BaseMeshProcessingTypes = 1000,
	BaseMeshProcessingTypesEditor = 2000,


	UserDefinedTypes = 10000
};


/**
 * This macro declares the type identifier for a GeometryFlow data type.
 * Generally TypeID is declared in an enum, like EDataTypes above, or EMeshProcessingDataTypes in GeometryFlowMeshProcessing.
 * Standard usage is to include at the top of the relevant class/struct definition.
 */
#define DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(TypeID) static constexpr int DataTypeIdentifier = (int)TypeID;



}	// end namespace GeometryFlow
}	// end namespace UE
