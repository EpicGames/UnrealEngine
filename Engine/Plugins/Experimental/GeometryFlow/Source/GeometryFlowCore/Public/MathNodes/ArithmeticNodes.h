// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseNodes/BinaryOpLambdaNode.h"
#include "GeometryFlowTypes.h"
#include "GeometryFlowNode.h"


namespace UE
{
namespace GeometryFlow
{

	template<typename T, int StorageTypeIdentifier>
	class TBinaryOpAddNode : public TBinaryOpLambdaNode<T, StorageTypeIdentifier>
	{
	public:
		TBinaryOpAddNode() : TBinaryOpLambdaNode<T, StorageTypeIdentifier>([](const T& A, const T& B) { return A + B; })
		{}
	};

	class FAddFloatNode : public TBinaryOpAddNode<float, (int)EDataTypes::Float>
	{
		static constexpr int NodeVersion = 1;
		GEOMETRYFLOW_NODE_INTERNAL(FAddFloatNode, NodeVersion, FNode)
	};


}	// end namespace GeometryFlow
}	// end namespace UE
