// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryFlowMeshProcessing : ModuleRules
{
	public GeometryFlowMeshProcessing(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"GeometryCore",
				"GeometryAlgorithms",
				"DynamicMesh",
				"GeometryFlowCore",
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
