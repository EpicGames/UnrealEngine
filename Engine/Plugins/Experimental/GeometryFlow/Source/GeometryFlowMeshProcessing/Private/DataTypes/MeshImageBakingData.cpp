// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTypes/MeshImageBakingData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshImageBakingData)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;

void FMakeMeshBakingCacheNode::Evaluate(
	const FNamedDataMap& DatasIn,
	FNamedDataMap& DatasOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	if (ensure(DatasOut.Contains(OutParamCache())))
	{
		bool bAllInputsValid = true;
		bool bRecomputeRequired = (IsOutputAvailable(OutParamCache()) == false);
		TSafeSharedPtr<IData> DetailMeshArg = FindAndUpdateInputForEvaluate(InParamDetailMesh(), DatasIn, bRecomputeRequired, bAllInputsValid);
		TSafeSharedPtr<IData> TargetMeshArg = FindAndUpdateInputForEvaluate(InParamTargetMesh(), DatasIn, bRecomputeRequired, bAllInputsValid);
		TSafeSharedPtr<IData> SettingsArg = FindAndUpdateInputForEvaluate(InParamSettings(), DatasIn, bRecomputeRequired, bAllInputsValid);
		if (bAllInputsValid)
		{
			if (bRecomputeRequired)
			{
				// always make a copy of settings
				FMeshMakeBakingCacheSettings Settings;
				SettingsArg->GetDataCopy(Settings, FMeshMakeBakingCacheSettings::DataTypeIdentifier);

				// todo: support stealing these meshes if they are mutable?
				bool bDetailIsMeshMutable = DatasIn.GetDataFlags(InParamDetailMesh()).bIsMutableData;
				bool bTargetIsMeshMutable = DatasIn.GetDataFlags(InParamTargetMesh()).bIsMutableData;


				TUniquePtr<FMeshBakingCache> NewCache = MakeUnique<FMeshBakingCache>();
				NewCache->DetailMesh = DetailMeshArg->GetDataConstRef<FDynamicMesh3>((int)EMeshProcessingDataTypes::DynamicMesh);
				NewCache->DetailSpatial = FDynamicMeshAABBTree3(&NewCache->DetailMesh, true);
				NewCache->TargetMesh = TargetMeshArg->GetDataConstRef<FDynamicMesh3>((int)EMeshProcessingDataTypes::DynamicMesh);

				NewCache->BakeCache.SetDetailMesh(&NewCache->DetailMesh, &NewCache->DetailSpatial);
				NewCache->BakeCache.SetBakeTargetMesh(&NewCache->TargetMesh);

				const int32 Width = FMath::Abs(Settings.Dimensions.X);
				const int32 Height = FMath::Abs(Settings.Dimensions.Y);
				NewCache->BakeCache.SetDimensions(FImageDimensions(Width, Height));
				NewCache->BakeCache.SetUVLayer(Settings.UVLayer);
				NewCache->BakeCache.SetThickness(Settings.Thickness);

				NewCache->BakeCache.ValidateCache();

				TSafeSharedPtr<FMeshBakingCacheData> Result = MakeShared<FMeshBakingCacheData, ESPMode::ThreadSafe>(MoveTemp(NewCache));
				SetOutput(OutParamCache(), Result);

				EvaluationInfo->CountCompute(this);
			}
			DatasOut.SetData(OutParamCache(), GetOutput(OutParamCache()));
		}
	}
}
