// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFlowMeshProcessingModule.h"
#include "Modules/ModuleManager.h"
#include "MeshProcessingNodeRegistration.h"

DEFINE_LOG_CATEGORY(LogGeometryFlowMeshProcessing);

#define LOCTEXT_NAMESPACE "FGeometryFlowMeshProcessingModule"

void FGeometryFlowMeshProcessingModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	
	UE::GeometryFlow::FMeshProcessingNodeRegistration::RegisterNodes();
} 

void FGeometryFlowMeshProcessingModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryFlowMeshProcessingModule, GeometryFlowMeshProcessing)
