// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshBakingNodes/BakeMeshMultiTextureNode.h"
#include "GeometryFlowNodeUtil.h"

#include "Sampling/MeshResampleImageBaker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeMeshMultiTextureNode)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;

void FBakeMeshMultiTextureNode::Evaluate(
	const FNamedDataMap& DatasIn,
	FNamedDataMap& DatasOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	if (ensure(DatasOut.Contains(OutParamTextureImage())))
	{
		bool bAllInputsValid = true;
		bool bRecomputeRequired = (IsOutputAvailable(OutParamTextureImage()) == false);
		TSafeSharedPtr<IData> BakeCacheArg = FindAndUpdateInputForEvaluate(InParamBakeCache(), DatasIn, bRecomputeRequired, bAllInputsValid);
		TSafeSharedPtr<IData> MaterialToTextureMapArg = FindAndUpdateInputForEvaluate(InParamMaterialTextures(), DatasIn, bRecomputeRequired, bAllInputsValid);
		TSafeSharedPtr<IData> SettingsArg = FindAndUpdateInputForEvaluate(InParamSettings(), DatasIn, bRecomputeRequired, bAllInputsValid);

		if (bAllInputsValid)
		{
			if (bRecomputeRequired)
			{
				// always make a copy of settings
				FBakeMeshMultiTextureSettings Settings;
				SettingsArg->GetDataCopy(Settings, FBakeMeshMultiTextureSettings::DataTypeIdentifier);

				const FMeshBakingCache& BakeCacheContainer = BakeCacheArg->GetDataConstRef<FMeshBakingCache>((int)EMeshProcessingDataTypes::BakingCache);
				const FDynamicMesh3& DetailMesh = BakeCacheContainer.DetailMesh;
				check(DetailMesh.HasAttributes() && DetailMesh.Attributes()->NumUVLayers() > Settings.DetailUVLayer);
				const FDynamicMeshUVOverlay* DetailUVOverlay = DetailMesh.Attributes()->GetUVLayer(Settings.DetailUVLayer);

				const FMaterialIDToTextureMap& SourceMaterialTextures = 
					MaterialToTextureMapArg->GetDataConstRef<FMaterialIDToTextureMap>(FMaterialIDToTextureMap::DataTypeIdentifier);

				FMeshMultiResampleImageBaker Baker;
				Baker.SetCache(&BakeCacheContainer.BakeCache);
				Baker.DetailUVOverlay = DetailUVOverlay;
				Baker.MultiTextures = SourceMaterialTextures.MaterialIDTextureMap;
				Baker.Bake();

				FTextureImage ResultImage;
				ResultImage.Image = MoveTemp(*Baker.TakeResult());

				SetOutput(OutParamTextureImage(), MakeMovableData<FTextureImage>(MoveTemp(ResultImage)) );

				EvaluationInfo->CountCompute(this);
			}
			DatasOut.SetData(OutParamTextureImage(), GetOutput(OutParamTextureImage()));
		}
	}
}
