// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshBakingNodes/BakeMeshNormalMapNode.h"
#include "GeometryFlowNodeUtil.h"

#include "Sampling/MeshNormalMapBaker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeMeshNormalMapNode)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;

void FBakeMeshNormalMapNode::Evaluate(
	const FNamedDataMap& DatasIn,
	FNamedDataMap& DatasOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	if (ensure(DatasOut.Contains(OutParamNormalMap())))
	{
		bool bAllInputsValid = true;
		bool bRecomputeRequired = (IsOutputAvailable(OutParamNormalMap()) == false);
		TSafeSharedPtr<IData> BakeCacheArg = FindAndUpdateInputForEvaluate(InParamBakeCache(), DatasIn, bRecomputeRequired, bAllInputsValid);
		TSafeSharedPtr<IData> TangentsArg = FindAndUpdateInputForEvaluate(InParamTangents(), DatasIn, bRecomputeRequired, bAllInputsValid);
		TSafeSharedPtr<IData> SettingsArg = FindAndUpdateInputForEvaluate(InParamSettings(), DatasIn, bRecomputeRequired, bAllInputsValid);
		if (bAllInputsValid)
		{
			if (bRecomputeRequired)
			{
				// always make a copy of settings
				FBakeMeshNormalMapSettings Settings;
				SettingsArg->GetDataCopy(Settings, FBakeMeshNormalMapSettings::DataTypeIdentifier);

				const FMeshTangentsd& BaseTangents = TangentsArg->GetDataConstRef<FMeshTangentsd>((int)EMeshProcessingDataTypes::MeshTangentSet);
				const FMeshBakingCache& BakeCacheContainer = BakeCacheArg->GetDataConstRef<FMeshBakingCache>((int)EMeshProcessingDataTypes::BakingCache);

				FMeshNormalMapBaker NormalBaker;
				NormalBaker.SetCache( &BakeCacheContainer.BakeCache );
				NormalBaker.BaseMeshTangents = &BaseTangents;
				NormalBaker.Bake();

				FNormalMapImage ResultImage;
				ResultImage.Image = MoveTemp(*NormalBaker.TakeResult());

				SetOutput(OutParamNormalMap(), MakeMovableData<FNormalMapImage>(MoveTemp(ResultImage)));

				EvaluationInfo->CountCompute(this);
			}
			DatasOut.SetData(OutParamNormalMap(), GetOutput(OutParamNormalMap()));
		}
	}
}
