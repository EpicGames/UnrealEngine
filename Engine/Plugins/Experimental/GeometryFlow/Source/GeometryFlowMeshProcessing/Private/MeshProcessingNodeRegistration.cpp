// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodeRegistration.h"
#include "GeometryFlowNodeFactory.h"


#include "DataTypes/CollisionGeometryData.h"
#include "DataTypes/DynamicMeshData.h"
#include "DataTypes/IndexSetsData.h"
#include "DataTypes/MeshImageBakingData.h"
#include "DataTypes/NormalMapData.h"
#include "DataTypes/WeightMapData.h"
#include "DataTypes/TextureImageData.h"

#include "MeshBakingNodes/BakeMeshMultiTextureNode.h"
#include "MeshBakingNodes/BakeMeshNormalMapNode.h"
#include "MeshBakingNodes/BakeMeshTextureImageNode.h"

#include "MeshDecompositionNodes/MakeTriangleSetsNode.h"

#include "MeshProcessingNodes/CompactMeshNode.h"
#include "MeshProcessingNodes/GenerateConvexHullMeshNode.h"

#include "MeshProcessingNodes/MeshDeleteTrianglesNode.h"
#include "MeshProcessingNodes/MeshMakeCleanGeometryNode.h"
#include "MeshProcessingNodes/MeshNormalFlowNode.h"
#include "MeshProcessingNodes/MeshNormalsNodes.h"
#include "MeshProcessingNodes/MeshRecalculateUVsNode.h"

#include "MeshProcessingNodes/MeshRepackUVsNode.h"
#include "MeshProcessingNodes/MeshSimplifyNode.h"
#include "MeshProcessingNodes/MeshSolidifyNode.h"

#include "MeshProcessingNodes/MeshTangentsNodes.h"
#include "MeshProcessingNodes/MeshThickenNode.h"
#include "MeshProcessingNodes/MeshVoxMorphologyNode.h"
#include "MeshProcessingNodes/TransferMeshMaterialIDsNode.h"



#include "PhysicsNodes/GenerateSimpleCollisionNode.h"

#include "MeshProcessingNodes/MeshProcessingDataTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "BaseNodes/SwitchNode.h"

namespace UE
{
namespace GeometryFlow
{

																			  
void FMeshProcessingNodeRegistration::RegisterNodes()
{

	FString CategoryName("Data Types");
	//  DataTypes / 
	GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(CollisionGeometry)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(CollisionGeometryTransfer)

	GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(DynamicMesh)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(DynamicMeshTransfer)

	GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(IndexSets)

	GEOMETRYFLOW_REGISTER_NODE_TYPE(MakeMeshBakingCache)
	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(MeshMakeBakingCache)

	GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(NormalMapImage)
	GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(WeightMap)

	GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(TextureImage)
	GEOMETRYFLOW_REGISTER_BASIC_TYPES_NODE(MaterialIDToTextureMap)


	// MeshBakingNodes/
	CategoryName = FString("Baking");
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(BakeMeshMultiTexture)
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(BakeMeshNormalMap)
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(BakeMeshTextureImage)


	// MeshDecompositionNodes/
	CategoryName = FString("Decomposition");
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MakeTriangleSetsFromMesh)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MakeTriangleSetsFromGroups)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MakeTriangleSetsFromConnectedComponents)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MakeTriangleSetsFromWeightMap)


	// MeshProcessingNodes/
	CategoryName = FString("Processing");
	GEOMETRYFLOW_REGISTER_NODE_TYPE(CompactMesh)
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(GenerateConvexHullMesh)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MeshDeleteTriangles)

	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(MeshMakeCleanGeometry)
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(MeshNormalFlow)

	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(Normals)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(ComputeMeshNormals)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(ComputeMeshPerVertexOverlayNormals)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(ComputeMeshPerVertexNormals)

	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(MeshRecalculateUVs)
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(MeshRepackUVs)

	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(Simplify)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(SimplifyMesh)

	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(Solidify)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(SolidifyMesh)

	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(Tangents)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(ComputeMeshTangents)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MeshTangentsTransfer)

	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(Thicken)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MeshThicken)


	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(VoxOffset)
	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(VoxClosure)
	GEOMETRYFLOW_REGISTER_SETTINGS_NODE_TYPE(VoxOpening)

	GEOMETRYFLOW_REGISTER_NODE_TYPE(VoxDilateMesh)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(VoxClosureMesh)
	GEOMETRYFLOW_REGISTER_NODE_TYPE(VoxOpeningMesh)

	GEOMETRYFLOW_REGISTER_NODE_TYPE(TransferMeshMaterialIDs)



	// PhysicsNodes /
	CategoryName = FString("Physics");
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(GenerateSimpleCollision)


	CategoryName = FString("Basic Types");
	typedef TSwitchNode<FDynamicMesh3, 4, (int)EMeshProcessingDataTypes::DynamicMesh> FMeshGeneratorSwitchNode;
	GEOMETRYFLOW_REGISTER_NODE_TYPE(MeshGeneratorSwitch)
}

#undef  GEOMETRYFLOW_REGISTER_NODE_TYPE
#undef  GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE
#undef  GEOMETRYFLOW_QUOTE
} }  // end namespaces


