// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodes/GenerateConvexHullMeshNode.h"
#include "Operations/MeshConvexHull.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateConvexHullMeshNode)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;

EGeometryFlowResult FGenerateConvexHullMeshNode::MakeConvexHullMesh(const FDynamicMesh3& MeshIn,
	const FGenerateConvexHullMeshSettings& Settings, 
	FDynamicMesh3& MeshOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	FMeshConvexHull Hull(&MeshIn);

	if (Settings.bPrefilterVertices)
	{
		FMeshConvexHull::GridSample(MeshIn, Settings.PrefilterGridResolution, Hull.VertexSet);
	}

	Hull.bPostSimplify = false;		// Mesh can be simplified later

	if (Hull.Compute(EvaluationInfo->Progress))
	{
		MeshOut = MoveTemp(Hull.ConvexHull);
	}

	if (EvaluationInfo->Progress->Cancelled())
	{
		return EGeometryFlowResult::OperationCancelled;
	}

	// TODO: What if Hull.Compute() fails but not because of cancellation? Should we have a EGeometryFlowResult::Failed?

	return EGeometryFlowResult::Ok;
}

