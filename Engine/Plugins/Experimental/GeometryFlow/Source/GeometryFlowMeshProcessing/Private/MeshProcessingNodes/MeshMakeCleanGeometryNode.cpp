// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodes/MeshMakeCleanGeometryNode.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshQueries.h"
#include "MeshBoundaryLoops.h"
#include "Operations/SimpleHoleFiller.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshMakeCleanGeometryNode)


using namespace UE::Geometry;
using namespace UE::GeometryFlow;

void FMeshMakeCleanGeometryNode::ProcessMesh(
	const FNamedDataMap& DatasIn,
	const FMeshMakeCleanGeometrySettings& SettingsIn,
	const FDynamicMesh3& MeshIn,
	FDynamicMesh3& MeshOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	MeshOut = MeshIn;
	ApplyMakeCleanGeometry(MeshOut, SettingsIn);
}


void FMeshMakeCleanGeometryNode::ProcessMeshInPlace(
	const FNamedDataMap& DatasIn,
	const FMeshMakeCleanGeometrySettings& Settings,
	FDynamicMesh3& MeshInOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	ApplyMakeCleanGeometry(MeshInOut, Settings);
}

// make a triangle fan for the hole - not a great area metric...
static double EstimateHoleFillArea(const FDynamicMesh3& Mesh, const FEdgeLoop& Loop)
{
	FVector3d Centroid = FVector3d::Zero();
	for (int32 vid : Loop.Vertices)
	{
		Centroid += Mesh.GetVertex(vid);
	}
	Centroid /= (double)Loop.Vertices.Num();

	double AreaSum = 0;
	for (int32 eid : Loop.Edges)
	{
		FIndex2i EdgeV = Mesh.GetEdgeV(eid);
		AreaSum += VectorUtil::Area(Mesh.GetVertex(EdgeV.A), Mesh.GetVertex(EdgeV.B), Centroid);
	}
	return AreaSum;
}


void FMeshMakeCleanGeometryNode::ApplyMakeCleanGeometry(FDynamicMesh3& Mesh, const FMeshMakeCleanGeometrySettings& Settings)
{
	if (Mesh.HasAttributes())
	{
		if (Settings.bDiscardAllAttributes)
		{
			Mesh.DiscardAttributes();
		}
		else
		{
			if (Settings.bClearUVs)
			{
				Mesh.Attributes()->SetNumUVLayers(0);
			}
			if (Settings.bClearVertexColors)
			{
				Mesh.Attributes()->DisablePrimaryColors();
			}
			if (Settings.bClearTangents && Mesh.Attributes()->NumNormalLayers() > 1)
			{
				Mesh.Attributes()->DisableTangents();
			}
			if (Settings.bClearNormals)
			{
				Mesh.Attributes()->SetNumNormalLayers(0);
			}
			if (Settings.bClearMaterialIDs)
			{
				Mesh.Attributes()->DisableMaterialID();
			}
		}
	}

	if (Settings.FillHolesEdgeCountThresh > 0 || Settings.FillHolesEstimatedAreaFraction > 0.0)
	{
		double MeshArea = 0.0;
		if (Settings.FillHolesEstimatedAreaFraction > 0.0)
		{
			FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(Mesh);
			MeshArea = VolArea.Y;
		}

		FMeshBoundaryLoops BoundaryLoops(&Mesh);
		int32 InitialHoleCount = BoundaryLoops.Loops.Num();

		for (FEdgeLoop& Loop : BoundaryLoops.Loops)
		{
			if (Mesh.IsBoundaryEdge(Loop.Edges[0]))
			{
				bool bFill = (Loop.Edges.Num() < Settings.FillHolesEdgeCountThresh);
				if (bFill == false && Settings.FillHolesEstimatedAreaFraction)
				{
					double HoleAreaEst = EstimateHoleFillArea(Mesh, Loop);
					bFill = ((HoleAreaEst / MeshArea) < Settings.FillHolesEstimatedAreaFraction);
				}

				if (bFill)
				{
					FSimpleHoleFiller Filler(&Mesh, Loop);
					Filler.Fill();
				}
			}
		}

		//UE_LOGF(LogGeometry, Warning, "GeometryFlow::MeshMakeCleanGeometryNode: Reduced from %d holes to %d holes", BoundaryLoops.Loops.Num(), FMeshBoundaryLoops(&Mesh).Loops.Num());
	}

	if (Settings.bOutputMeshVertexNormals)
	{
		FMeshNormals::QuickComputeVertexNormals(Mesh, false);
	}
	else
	{
		Mesh.DiscardVertexNormals();
	}

	if (Settings.bOutputOverlayVertexNormals)
	{
		if (Mesh.HasAttributes())
		{
			if (Mesh.Attributes()->NumNormalLayers() < 1)
			{
				Mesh.Attributes()->SetNumNormalLayers(1);
			}
			FMeshNormals::InitializeOverlayToPerVertexNormals(Mesh.Attributes()->PrimaryNormals(), Settings.bOutputMeshVertexNormals);
		}
	}
}
