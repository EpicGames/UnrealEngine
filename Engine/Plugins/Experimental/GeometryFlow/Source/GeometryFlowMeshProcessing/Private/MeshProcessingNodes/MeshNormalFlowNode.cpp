// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodes/MeshNormalFlowNode.h"
#include "NormalFlowRemesher.h"
#include "DynamicMesh/MeshNormals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshNormalFlowNode)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;

namespace
{
	// TODO: This duplicates code in a private function in URemeshMeshTool
	double CalculateTargetEdgeLength(const FDynamicMesh3& Mesh, int TargetTriCount)
	{
		double InitialMeshArea = 0;
		for (int TriangleIndex : Mesh.TriangleIndicesItr())
		{
			InitialMeshArea += Mesh.GetTriArea(TriangleIndex);
		}

		double TargetTriArea = InitialMeshArea / (double)TargetTriCount;
		double EdgeLen = TriangleUtil::EquilateralEdgeLengthForArea(TargetTriArea);
		return (double)FMath::RoundToInt(EdgeLen * 100.0) / 100.0;
	}
}

void FMeshNormalFlowNode::CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid)
{
	FindAndUpdateInputForEvaluate(InParamTargetMesh(), DatasIn, bRecomputeRequired, bAllInputsValid);
}


void FMeshNormalFlowNode::DoNormalFlow(const FMeshNormalFlowSettings& SettingsIn,
									   const FDynamicMesh3& ProjectionTargetMesh,
									   bool bAttributesHaveBeenDiscarded,
									   FDynamicMesh3& EditMesh)
{
	FNormalFlowRemesher Remesher(&EditMesh);

	Remesher.MaxRemeshIterations = SettingsIn.MaxRemeshIterations;
	Remesher.NumExtraProjectionIterations = SettingsIn.NumExtraProjectionIterations;

	Remesher.bEnableSplits = SettingsIn.bSplits;
	Remesher.bEnableFlips = SettingsIn.bFlips;
	Remesher.bEnableCollapses = SettingsIn.bCollapses;

	double TargetEdgeLength = CalculateTargetEdgeLength(EditMesh, SettingsIn.TargetCount);
	Remesher.SetTargetEdgeLength(TargetEdgeLength);

	Remesher.bEnableSmoothing = (SettingsIn.SmoothingStrength > 0.0f);
	Remesher.SmoothSpeedT = SettingsIn.SmoothingStrength;
	Remesher.SmoothType = FromUEnum(SettingsIn.SmoothingType);

	if (!SettingsIn.bDiscardAttributes)
	{
		Remesher.FlipMetric = (SettingsIn.SmoothingType == ToUEnum(FRemesher::ESmoothTypes::Uniform)) ?
			FRemesher::EFlipMetric::OptimalValence : FRemesher::EFlipMetric::MinEdgeLength;
	}

	Remesher.bPreventNormalFlips = SettingsIn.bPreventNormalFlips;
	Remesher.DEBUG_CHECK_LEVEL = 0;

	FMeshConstraints Constraints;
	FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, EditMesh,
														 FromUEnum(SettingsIn.MeshBoundaryConstraints),
														 FromUEnum(SettingsIn.GroupBorderConstraints),
														 FromUEnum(SettingsIn.MaterialBorderConstraints),
														 true, !SettingsIn.bPreserveSharpEdges);

	Remesher.SetExternalConstraints(MoveTemp(Constraints));


	FDynamicMeshAABBTree3 ProjectionTargetSpatial(&ProjectionTargetMesh, true);
	FMeshProjectionTarget ProjTarget(&ProjectionTargetMesh, &ProjectionTargetSpatial);
	Remesher.SetProjectionTarget(&ProjTarget);

	if (SettingsIn.bDiscardAttributes && !bAttributesHaveBeenDiscarded)
	{
		EditMesh.DiscardAttributes();
	}

	// Go
	Remesher.BasicRemeshPass();

	if (!EditMesh.HasAttributes())
	{
		FMeshNormals::QuickComputeVertexNormals(EditMesh);
	}
	else
	{
		FMeshNormals::QuickRecomputeOverlayNormals(EditMesh);
	}
}

void FMeshNormalFlowNode::ProcessMesh(
	const FNamedDataMap& DatasIn,
	const FMeshNormalFlowSettings& SettingsIn,
	const FDynamicMesh3& MeshIn,
	FDynamicMesh3& MeshOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	TSafeSharedPtr<IData> ProjectionTargetMeshArg = DatasIn.FindData(InParamTargetMesh());
	FDynamicMesh3 ProjectionTargetMesh;
	ProjectionTargetMeshArg->GetDataCopy<FDynamicMesh3>(ProjectionTargetMesh, (int)EMeshProcessingDataTypes::DynamicMesh);

	bool bDiscardAttributesImmediately = SettingsIn.bDiscardAttributes && !SettingsIn.bPreserveSharpEdges;
	MeshOut.Copy(MeshIn, true, true, true, !bDiscardAttributesImmediately);

	DoNormalFlow(SettingsIn, ProjectionTargetMesh, bDiscardAttributesImmediately, MeshOut);
}


void FMeshNormalFlowNode::ProcessMeshInPlace(
	const FNamedDataMap& DatasIn,
	const FMeshNormalFlowSettings& SettingsIn,
	FDynamicMesh3& MeshInOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	// TODO: In this case, can we call IData::GiveTo instead of GetDataCopy for the projection target?
	TSafeSharedPtr<IData> ProjectionTargetMeshArg = DatasIn.FindData(InParamTargetMesh());
	FDynamicMesh3 ProjectionTargetMesh;
	ProjectionTargetMeshArg->GetDataCopy<FDynamicMesh3>(ProjectionTargetMesh, (int)EMeshProcessingDataTypes::DynamicMesh);

	constexpr bool bAttributesHaveBeenDiscarded = false;
	DoNormalFlow(SettingsIn, ProjectionTargetMesh, bAttributesHaveBeenDiscarded, MeshInOut);
}
