// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodes/MeshRecalculateUVsNode.h"

#include "Selections/MeshConnectedComponents.h"
#include "Parameterization/DynamicMeshUVEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshRecalculateUVsNode)


using namespace UE::Geometry;
using namespace UE::GeometryFlow;


void FMeshRecalculateUVsNode::RecalculateUVsOnMesh(FDynamicMesh3& EditMesh, const FMeshRecalculateUVsSettings& Settings)
{
	FDynamicMeshUVEditor UVEditor(&EditMesh, Settings.UVLayer, true);
	FDynamicMeshUVOverlay* UVOverlay = UVEditor.GetOverlay();

	FMeshConnectedComponents UVComponents(&EditMesh);
	UVComponents.FindConnectedTriangles([&](int32 Triangle0, int32 Triangle1) {
		return UVOverlay->AreTrianglesConnected(Triangle0, Triangle1);
	});

	for (int32 k = 0; k < UVComponents.Num(); ++k)
	{
		FMeshConnectedComponents::FComponent& Component = UVComponents.GetComponent(k);
		if (!ensure(Component.Indices.Num() > 0))
		{
			continue;		// how??
		}

		FUVEditResult EditResult;

		ERecalculateUVsUnwrapType UseUnwrapType = FromUEnum(Settings.UnwrapType);
		if (UseUnwrapType == ERecalculateUVsUnwrapType::Auto)
		{
			UseUnwrapType = ERecalculateUVsUnwrapType::Conformal;
		}

		bool bComplete = false;
		if (UseUnwrapType == ERecalculateUVsUnwrapType::Conformal)
		{
			bComplete = UVEditor.SetTriangleUVsFromFreeBoundaryConformal(Component.Indices, &EditResult);
			// if this fails we will fall back to ExpMap
		}

		if (!bComplete)
		{
			EditResult = FUVEditResult();
			if (UVEditor.SetTriangleUVsFromExpMap(Component.Indices) == false)
			{
				// if we somehow failed at conformal, fallback to trivial planar projection
				UVEditor.SetTriangleUVsFromProjection(Component.Indices, FFrame3d());
			}
		}

	}
}



