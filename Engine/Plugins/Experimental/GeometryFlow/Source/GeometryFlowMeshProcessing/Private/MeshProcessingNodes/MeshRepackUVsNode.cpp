// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodes/MeshRepackUVsNode.h"

#include "Parameterization/MeshUVPacking.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshRepackUVsNode)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;


void FMeshRepackUVsNode::RepackUVsForMesh(FDynamicMesh3& EditMesh, const FMeshRepackUVsSettings& Settings)
{
	if ( !(EditMesh.HasAttributes() && EditMesh.Attributes()->GetUVLayer(Settings.UVLayer)) )
	{
		return;
	}

	if (EditMesh.TriangleCount() == 0)
	{
		return;
	}

	FDynamicMeshUVOverlay* UVLayer = EditMesh.Attributes()->GetUVLayer(Settings.UVLayer);

	UVLayer->SplitBowties();

	FDynamicMeshUVPacker Packer(UVLayer);
	Packer.TextureResolution = Settings.TextureResolution;
	Packer.GutterSize = Settings.GutterSize;
	Packer.bAllowFlips = Settings.bAllowFlips;

	bool bOK = Packer.StandardPack();
	if (!ensure(bOK)) { return; }


	if (Settings.UVScale != FVector2f::One() || Settings.UVTranslation != FVector2f::Zero())
	{
		for (int ElementID : UVLayer->ElementIndicesItr())
		{
			FVector2f UV = UVLayer->GetElement(ElementID);
			UV.X *= Settings.UVScale.X;
			UV.Y *= Settings.UVScale.Y;
			UV += Settings.UVTranslation;
			UVLayer->SetElement(ElementID, UV);
		}
	}

}



