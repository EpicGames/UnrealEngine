// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodes/MeshThickenNode.h"
#include "Operations/DisplaceMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshThickenNode)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;

void FMeshThickenNode::ProcessMesh(
	const FNamedDataMap& DatasIn,
	const FMeshThickenSettings& SettingsIn,
	const FDynamicMesh3& MeshIn,
	FDynamicMesh3& MeshOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	TSafeSharedPtr<IData> WeightMapMeshArg = DatasIn.FindData(InParamWeightMap());
	FWeightMap WeightMap;
	WeightMapMeshArg->GetDataCopy<FWeightMap>(WeightMap, (int)EMeshProcessingDataTypes::WeightMap);

	MeshOut = MeshIn;
	ApplyThicken(MeshOut, SettingsIn, WeightMap.Weights);
}


void FMeshThickenNode::ProcessMeshInPlace(
	const FNamedDataMap& DatasIn,
	const FMeshThickenSettings& Settings,
	FDynamicMesh3& MeshInOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	TSafeSharedPtr<IData> WeightMapMeshArg = DatasIn.FindData(InParamWeightMap());
	FWeightMap WeightMap;
	WeightMapMeshArg->GetDataCopy<FWeightMap>(WeightMap, (int)EMeshProcessingDataTypes::WeightMap);

	ApplyThicken(MeshInOut, Settings, WeightMap.Weights);
}


void FMeshThickenNode::ApplyThicken(FDynamicMesh3& Mesh, const FMeshThickenSettings& Settings, const TArray<float>& VertexWeights)
{
	if (VertexWeights.Num() == 0)
	{
		return;
	}

	// TODO: Accept existing normals and/or allow user to choose type of normal to compute
	FMeshNormals Normals(&Mesh);
	Normals.ComputeVertexNormals();

	FDisplaceMesh::DisplaceMeshWithVertexWeights(Mesh, Normals, VertexWeights, Settings.ThickenAmount);
}
