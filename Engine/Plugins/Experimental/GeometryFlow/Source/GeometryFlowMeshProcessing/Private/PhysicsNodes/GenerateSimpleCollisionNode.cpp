// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsNodes/GenerateSimpleCollisionNode.h"
#include "Operations/MeshConvexHull.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "Async/ParallelFor.h"
#include "DynamicSubmesh3.h"
#include "Util/ProgressCancel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateSimpleCollisionNode)

using namespace UE::Geometry;
using namespace UE::GeometryFlow;


namespace GenerateSimpleCollisionNodeHelpers
{
	TArray<int> SetIntersection(const TArray<int>& A, const TArray<int>& B)
	{
		TSet<int> SA(A);
		TSet<int> SB(B);
		return SA.Intersect(SB).Array();
	}

	void GenerateConvexHulls(const FDynamicMesh3& Mesh,
							 const FIndexSets& IndexData,
							 const FGenerateSimpleCollisionSettings& Settings,
							 FCollisionGeometry& Result,
							 FProgressCancel* Progress)
	{
		int32 NumShapes = IndexData.IndexSets.Num();
		TArray<FConvexShape3d> Convexes;
		TArray<bool> ConvexOk;

		if (NumShapes == 0)
		{
			// No index sets given. Create a single convex hull from the whole mesh.
			Convexes.SetNum(1);
			ConvexOk.SetNum(1);
			ConvexOk[0] = false;
			FMeshConvexHull Hull(&Mesh);

			if (Settings.ConvexHullSettings.bPrefilterVertices)
			{
				FMeshConvexHull::GridSample(Mesh, Settings.ConvexHullSettings.PrefilterGridResolution, Hull.VertexSet);
			}

			Hull.bPostSimplify = Settings.ConvexHullSettings.SimplifyToTriangleCount > 0;
			Hull.SimplifyOptions.MaxTargetFaceCount = Settings.ConvexHullSettings.SimplifyToTriangleCount;
			Hull.SimplifyOptions.GeometricToleranceAfterTarget = Settings.ConvexHullSettings.GeometricToleranceAfterTarget;
			if (Hull.Compute(Progress))
			{
				FConvexShape3d NewConvex;
				NewConvex.Mesh = MoveTemp(Hull.ConvexHull);
				Convexes[0] = MoveTemp(NewConvex);
				ConvexOk[0] = true;
			}
		}
		else
		{
			// Create one hull for each index set.
			Convexes.SetNum(NumShapes);
			ConvexOk.SetNum(NumShapes);
			ParallelFor(NumShapes, [&](int32 k)
			{
				ConvexOk[k] = false;
				FMeshConvexHull Hull(&Mesh);

				if (Settings.ConvexHullSettings.bPrefilterVertices)
				{
					TArray<int> SampledVertices;
					FMeshConvexHull::GridSample(Mesh, Settings.ConvexHullSettings.PrefilterGridResolution, SampledVertices);
					TArray<int> IndexSetVertices;
					UE::Geometry::TriangleToVertexIDs(&Mesh, IndexData.IndexSets[k], IndexSetVertices);
					Hull.VertexSet = SetIntersection(SampledVertices, IndexSetVertices);
				}
				else
				{
					UE::Geometry::TriangleToVertexIDs(&Mesh, IndexData.IndexSets[k], Hull.VertexSet);
				}

				Hull.bPostSimplify = Settings.ConvexHullSettings.SimplifyToTriangleCount > 0;
				Hull.SimplifyOptions.MaxTargetFaceCount = Settings.ConvexHullSettings.SimplifyToTriangleCount;
				Hull.SimplifyOptions.GeometricToleranceAfterTarget = Settings.ConvexHullSettings.GeometricToleranceAfterTarget;
				if (Hull.Compute(Progress))
				{
					FConvexShape3d NewConvex;
					NewConvex.Mesh = MoveTemp(Hull.ConvexHull);
					Convexes[k] = MoveTemp(NewConvex);
					ConvexOk[k] = true;
				}
			});
		}

		Result.Geometry.Convexes = MoveTemp(Convexes);
	}

	void GenerateSubMeshes(const FDynamicMesh3& Mesh, const FIndexSets& TriangleIndexSets, TArray<FDynamicMesh3>& OutSubMeshes)
	{
		if (TriangleIndexSets.IndexSets.Num() == 0)
		{
			OutSubMeshes.Add(Mesh);
		}
		else
		{
			for (const TArray<int32>& Set : TriangleIndexSets.IndexSets)
			{
				FDynamicSubmesh3 Sub(&Mesh, Set);
				OutSubMeshes.Add(Sub.GetSubmesh());
			}
		}
	}

	FMeshSimpleShapeApproximation::EProjectedHullAxisMode FromUEnum(const EGeometryFlow_ProjectedHullAxisMode& Value) { return static_cast<FMeshSimpleShapeApproximation::EProjectedHullAxisMode>(Value); }
} // namespace GenerateSimpleCollisionNodeHelpers


EGeometryFlowResult FGenerateSimpleCollisionNode::EvaluateInternal(const FDynamicMesh3& Mesh,
																   const FIndexSets& IndexData,
																   const FGenerateSimpleCollisionSettings& Settings,
																   TUniquePtr<FEvaluationInfo>& EvaluationInfo,
																   FCollisionGeometry& OutCollisionGeometry)
{
	 FMeshSimpleShapeApproximation ShapeApproximator;

	 // Don't cache anything, just compute when asked to
	 ShapeApproximator.bDetectSpheres = false;
	 ShapeApproximator.bDetectBoxes = false;
	 ShapeApproximator.bDetectCapsules = false;
	 ShapeApproximator.bDetectConvexes = false;

	 TArray<FDynamicMesh3> SubMeshes;
	 GenerateSimpleCollisionNodeHelpers::GenerateSubMeshes(Mesh, IndexData, SubMeshes);

	 if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
	 {
		 return EGeometryFlowResult::OperationCancelled;
	 }

	 TArray<const FDynamicMesh3*> SubMeshPointers;
	 for (FDynamicMesh3& SubMesh : SubMeshes)
	 {
		 SubMeshPointers.Add(&SubMesh);
	 }
	 ShapeApproximator.InitializeSourceMeshes(SubMeshPointers);

	 if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
	 {
		 return EGeometryFlowResult::OperationCancelled;
	 }

	 FProgressCancel* Progress = EvaluationInfo ? EvaluationInfo->Progress : nullptr;

	 switch (Settings.Type)
	 {
	 case EGeometryFlow_SimpleCollisionGeometryType::AlignedBoxes:
		 ShapeApproximator.bDetectBoxes = true;
		 ShapeApproximator.Generate_AlignedBoxes(OutCollisionGeometry.Geometry);
		 break;
	 case EGeometryFlow_SimpleCollisionGeometryType::OrientedBoxes:
		 ShapeApproximator.bDetectBoxes = true;
		 ShapeApproximator.Generate_OrientedBoxes(OutCollisionGeometry.Geometry, Progress);
		 break;
	 case EGeometryFlow_SimpleCollisionGeometryType::MinimalSpheres:
		 ShapeApproximator.bDetectSpheres = true;
		 ShapeApproximator.Generate_MinimalSpheres(OutCollisionGeometry.Geometry);
		 break;
	 case EGeometryFlow_SimpleCollisionGeometryType::Capsules:
		 ShapeApproximator.bDetectCapsules = true;
		 ShapeApproximator.Generate_Capsules(OutCollisionGeometry.Geometry);
		 break;
	 case EGeometryFlow_SimpleCollisionGeometryType::ConvexHulls:
		 GenerateSimpleCollisionNodeHelpers::GenerateConvexHulls(Mesh, IndexData, Settings, OutCollisionGeometry, Progress);
		 break;
	 case EGeometryFlow_SimpleCollisionGeometryType::SweptHulls:
		 ShapeApproximator.bDetectConvexes = true;
		 ShapeApproximator.bSimplifyHulls = Settings.SweptHullSettings.bSimplifyPolygons;
		 ShapeApproximator.HullSimplifyTolerance = Settings.SweptHullSettings.HullTolerance;
		 ShapeApproximator.Generate_ProjectedHulls(OutCollisionGeometry.Geometry, GenerateSimpleCollisionNodeHelpers::FromUEnum(Settings.SweptHullSettings.SweepAxis));
		 break;
	 case EGeometryFlow_SimpleCollisionGeometryType::MinVolume:
		 ShapeApproximator.Generate_MinVolume(OutCollisionGeometry.Geometry);
		 break;
	 case EGeometryFlow_SimpleCollisionGeometryType::None:
		 break;
	 }

	 if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
	 {
		 return EGeometryFlowResult::OperationCancelled;
	 }

	 return EGeometryFlowResult::Ok;
}


void FGenerateSimpleCollisionNode::Evaluate(
	const FNamedDataMap& DatasIn,
	FNamedDataMap& DatasOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	if (ensure(DatasOut.Contains(OutParamGeometry())))
	{
		bool bAllInputsValid = true;
		bool bRecomputeRequired = (IsOutputAvailable(OutParamGeometry()) == false);

		TSafeSharedPtr<IData> MeshArg = FindAndUpdateInputForEvaluate(InParamMesh(),
																	  DatasIn,
																	  bRecomputeRequired,
																	  bAllInputsValid);
		const FDynamicMesh3& Mesh = MeshArg->GetDataConstRef<FDynamicMesh3>((int)EMeshProcessingDataTypes::DynamicMesh);

		TSafeSharedPtr<IData> TriSetsArg = FindAndUpdateInputForEvaluate(InParamIndexSets(), 
																		 DatasIn, 
																		 bRecomputeRequired, 
																		 bAllInputsValid);
		const FIndexSets& IndexData = TriSetsArg->GetDataConstRef<FIndexSets>((int)EMeshProcessingDataTypes::IndexSets);

		TSafeSharedPtr<IData> SettingsArg = FindAndUpdateInputForEvaluate(InParamSettings(),
																		  DatasIn,
																		  bRecomputeRequired, bAllInputsValid);
		FGenerateSimpleCollisionSettings Settings;
		SettingsArg->GetDataCopy(Settings, FGenerateSimpleCollisionSettings::DataTypeIdentifier);

		if (bAllInputsValid)
		{
			if (bRecomputeRequired)
			{
				// if execution is interrupted by ProgressCancel, make sure this node is recomputed next time through
				ClearAllOutputs();

				FCollisionGeometry CollisionGeometry;
				EGeometryFlowResult Result = EvaluateInternal(Mesh, IndexData, Settings, EvaluationInfo, CollisionGeometry);

				if (Result == EGeometryFlowResult::OperationCancelled)
				{
					return;
				}

				SetOutput(OutParamGeometry(), MakeMovableData<FCollisionGeometry>(MoveTemp(CollisionGeometry)));
			}

			DatasOut.SetData(OutParamGeometry(), GetOutput(OutParamGeometry()));
		}
	}
}
