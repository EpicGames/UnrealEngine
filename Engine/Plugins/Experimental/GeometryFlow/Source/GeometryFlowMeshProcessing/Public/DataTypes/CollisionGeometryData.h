// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "MeshProcessingNodes/MeshProcessingDataTypes.h"

#include "ShapeApproximation/SimpleShapeSet3.h"

namespace UE::GeometryFlow { template <typename T, int StorageTypeIdentifier> class TTransferNode; }


namespace UE
{
namespace GeometryFlow
{


struct FCollisionGeometry
{
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(EMeshProcessingDataTypes::CollisionGeometry);

	UE::Geometry::FSimpleShapeSet3d Geometry;
};


// declares FDataCollisionGeometry, FCollisionGeometryInput, FCollisionGeometryOutput, FCollisionGeometrySourceNode
// @todo - add serialization.  
GEOMETRYFLOW_DECLARE_BASIC_TYPES_NULL_SERIALIZE(CollisionGeometry, FCollisionGeometry, (int)EMeshProcessingDataTypes::CollisionGeometry, 1);


typedef TTransferNode<FCollisionGeometry, (int)EMeshProcessingDataTypes::CollisionGeometry> FCollisionGeometryTransferNode;



}	// end namespace GeometryFlow
}	// end namespace UE
