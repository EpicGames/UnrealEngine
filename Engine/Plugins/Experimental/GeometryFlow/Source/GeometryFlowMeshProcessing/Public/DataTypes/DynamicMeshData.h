// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryFlowCoreNodes.h"
#include "BaseNodes/TransferNode.h"
#include "MeshProcessingNodes/MeshProcessingDataTypes.h"

namespace UE::Geometry { class FDynamicMesh3; }
namespace UE::GeometryFlow { template <typename T, int StorageTypeIdentifier> class TTransferNode; }

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

// declares FDataDynamicMesh, FDynamicMeshInput, FDynamicMeshOutput, FDynamicMeshSourceNode
template<>
struct TSerializationMethod<FDynamicMesh3> { static void Serialize(FArchive& Ar, FDynamicMesh3& Mesh) { Ar << Mesh; } };
typedef TMovableData<FDynamicMesh3, (int)EMeshProcessingDataTypes::DynamicMesh> FDataDynamicMesh;
typedef TBasicNodeInput<FDynamicMesh3, (int)EMeshProcessingDataTypes::DynamicMesh> FDynamicMeshInput;
typedef TBasicNodeOutput<FDynamicMesh3, (int)EMeshProcessingDataTypes::DynamicMesh> FDynamicMeshOutput;
class FDynamicMeshSourceNode : public  TSourceNodeBase<FDynamicMesh3, (int)EMeshProcessingDataTypes::DynamicMesh>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FDynamicMeshSourceNode, Version, FSourceNodeBase)
};


class FDynamicMeshTransferNode : public TTransferNode<FDynamicMesh3, (int)EMeshProcessingDataTypes::DynamicMesh>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FDynamicMeshTransferNode, Version, FNode)
};

}	// end namespace GeometryFlow
}	// end namespace UE
