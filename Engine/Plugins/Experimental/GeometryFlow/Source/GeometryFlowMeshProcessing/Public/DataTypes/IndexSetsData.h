// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "MeshProcessingNodes/MeshProcessingDataTypes.h"


namespace UE
{
namespace GeometryFlow
{


struct FIndexSets
{
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(EMeshProcessingDataTypes::IndexSets);

	TArray<TArray<int32>> IndexSets;


	//
	// utility functions
	//
	void AppendSet()
	{
		TArray<int32> Values;
		IndexSets.Add(Values);
	}

	void AppendSet(const TArray<int32>& SetValues)
	{
		IndexSets.Add(SetValues);
	}

	int32 NumSets() const { return IndexSets.Num(); }

	template<typename ListType>
	void GetAllValues(ListType& ValuesOut) const
	{
		for (const TArray<int32>& Set : IndexSets)
		{
			for (int32 Value : Set)
			{
				ValuesOut.Add(Value);
			}
		}
	}
};


// declares FDataIndexSets, FIndexSetsInput, FIndexSetsOutput, FIndexSetsSourceNode
template<> struct TSerializationMethod<FIndexSets> { static void Serialize(FArchive& Ar, FIndexSets& Data) { Ar << Data.IndexSets; } }; \
GEOMETRYFLOW_DECLARE_BASIC_TYPES_WO_SERIALIZATION(IndexSets, FIndexSets, (int)EMeshProcessingDataTypes::IndexSets, 1)






}	// end namespace GeometryFlow
}	// end namespace UE
