// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowImmutableData.h"
#include "DataTypes/DynamicMeshData.h"

#include "GeometryFlowMovableData.h"
#include "Sampling/MeshImageBakingCache.h"
#include "Image/ImageOccupancyMap.h"
#include "Image/ImageDimensions.h"


#include "MeshImageBakingData.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

USTRUCT()
struct FMeshMakeBakingCacheSettings
{

	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::MakeBakingCacheSettings);


	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FIntPoint Dimensions{ EForceInit::ForceInit };  //FImageDimensions Dimensions;
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 UVLayer = 0;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float Thickness = 0.1f;

};


namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

typedef FMeshMakeBakingCacheSettings FMeshMakeBakingCacheSettings;

struct FMeshBakingCache
{
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(EMeshProcessingDataTypes::BakingCache);

	// have to make copies of these - boo!
	FDynamicMesh3 DetailMesh;
	FDynamicMeshAABBTree3 DetailSpatial;
	FDynamicMesh3 TargetMesh;

	UE::Geometry::FMeshImageBakingCache BakeCache;
};


typedef TImmutableData<FMeshBakingCache, FMeshBakingCache::DataTypeIdentifier> FMeshBakingCacheData;


GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshMakeBakingCacheSettings, MeshMakeBakingCache, 1);





class FMakeMeshBakingCacheNode : public FNode
{
protected:
	using SettingsDataType = TMovableData<FMeshMakeBakingCacheSettings, FMeshMakeBakingCacheSettings::DataTypeIdentifier>;
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMakeMeshBakingCacheNode, Version, FNode)
public:
	static const FString InParamDetailMesh() { return TEXT("DetailMesh"); }
	static const FString InParamTargetMesh() { return TEXT("TargetMesh"); }
	static const FString InParamSettings() { return TEXT("Settings"); }

	static const FString OutParamCache() { return TEXT("BakeCache"); }

public:
	FMakeMeshBakingCacheNode()
	{
		AddInput(InParamDetailMesh(), MakeUnique<FDynamicMeshInput>());
		AddInput(InParamTargetMesh(), MakeUnique<FDynamicMeshInput>());
		AddInput(InParamSettings(), MakeUnique<TBasicNodeInput<FMeshMakeBakingCacheSettings, FMeshMakeBakingCacheSettings::DataTypeIdentifier>>());

		AddOutput(OutParamCache(), MakeUnique<TImmutableNodeOutput<FMeshBakingCache, FMeshBakingCache::DataTypeIdentifier>>() );
	}

	UE_API virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;
};








}	// end namespace GeometryFlow
}	// end 

#undef UE_API
