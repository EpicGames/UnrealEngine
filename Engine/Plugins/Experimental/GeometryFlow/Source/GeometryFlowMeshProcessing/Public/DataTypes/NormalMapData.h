// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "MeshProcessingNodes/MeshProcessingDataTypes.h"

#include "Image/ImageBuilder.h"


namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

struct FNormalMapImage
{
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(EMeshProcessingDataTypes::NormalMapImage);

	TImageBuilder<FVector3f> Image;
};


// declares FDataNormalMapImage, FNormalMapImageInput, FNormalMapImageOutput, FNormalMapImageSourceNode
// @todo - serialize, or reconsider this node as just being some form of graph output.
GEOMETRYFLOW_DECLARE_BASIC_TYPES_NULL_SERIALIZE(NormalMapImage, FNormalMapImage, (int)EMeshProcessingDataTypes::NormalMapImage, 1)




}	// end namespace GeometryFlow
}	// end namespace UE
