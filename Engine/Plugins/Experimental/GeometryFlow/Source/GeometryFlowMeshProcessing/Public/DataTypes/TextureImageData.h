// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "MeshProcessingNodes/MeshProcessingDataTypes.h"

#include "Image/ImageBuilder.h"

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

struct FTextureImage
{
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(EMeshProcessingDataTypes::TextureImage);

	TImageBuilder<FVector4f> Image;
};

// declares FDataTextureImage, FTextureImageInput, FTextureImageOutput, FTextureImageSourceNode
GEOMETRYFLOW_DECLARE_BASIC_TYPES_NULL_SERIALIZE(TextureImage, FTextureImage, (int)EMeshProcessingDataTypes::TextureImage, 1)


struct FMaterialIDToTextureMap
{
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(EMeshProcessingDataTypes::MaterialIDToTextureMap);

	TMap<int32, TSafeSharedPtr<TImageBuilder<FVector4f>>> MaterialIDTextureMap;
};

// @todo add some form of serialization 
GEOMETRYFLOW_DECLARE_BASIC_TYPES_NULL_SERIALIZE(MaterialIDToTextureMap, FMaterialIDToTextureMap, (int)EMeshProcessingDataTypes::MaterialIDToTextureMap, 1)


}	// end namespace GeometryFlow
}	// end namespace UE
