// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "MeshProcessingNodes/MeshProcessingDataTypes.h"


namespace UE
{
namespace GeometryFlow
{


struct FWeightMap
{
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(EMeshProcessingDataTypes::WeightMap);

	TArray<float> Weights;

	// TODO: Sparse map from vertex ID to weight value?
};


// @todo - serialize, or reconsider this node as just being some form of graph output.
template <> struct TSerializationMethod<FWeightMap> { static void Serialize(FArchive& Ar, FWeightMap& Data) { Ar << Data.Weights; } }; \
GEOMETRYFLOW_DECLARE_BASIC_TYPES_WO_SERIALIZATION(WeightMap, FWeightMap, (int)EMeshProcessingDataTypes::WeightMap, 1)



}	// end namespace GeometryFlow
}	// end namespace UE
