// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

GEOMETRYFLOWMESHPROCESSING_API DECLARE_LOG_CATEGORY_EXTERN(LogGeometryFlowMeshProcessing, Log, All);

class FGeometryFlowMeshProcessingModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
