// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "MeshBakingNodes/BakeMeshTextureImageNode.h"

#include "BakeMeshMultiTextureNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

class UTexture2D;

USTRUCT()
struct FBakeMeshMultiTextureSettings : public FBakeMeshTextureImageSettings
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::BakeMultiTextureSettings);
};


namespace UE
{
namespace GeometryFlow
{

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FBakeMeshMultiTextureSettings, BakeMeshMultiTexture, 1);


class FBakeMeshMultiTextureNode : public FNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FBakeMeshMultiTextureNode, Version, FNode)

protected:
	using SettingsDataType = TMovableData<FBakeMeshMultiTextureSettings, FBakeMeshMultiTextureSettings::DataTypeIdentifier>;

public:
	static const FString InParamBakeCache() { return TEXT("BakeCache"); }
	static const FString InParamMaterialTextures() { return TEXT("MaterialIDToTextureMap"); }
	static const FString InParamSettings() { return TEXT("Settings"); }
	static const FString OutParamTextureImage() { return TEXT("TextureImage"); }

public:
	FBakeMeshMultiTextureNode()
	{
		AddInput(InParamBakeCache(), MakeUnique<TImmutableNodeInput<FMeshBakingCache, FMeshBakingCache::DataTypeIdentifier>>());
		AddInput(InParamMaterialTextures(), MakeUnique<TBasicNodeInput<FMaterialIDToTextureMap, FMaterialIDToTextureMap::DataTypeIdentifier>>());
		AddInput(InParamSettings(), MakeUnique<TBasicNodeInput<FBakeMeshMultiTextureSettings, FBakeMeshMultiTextureSettings::DataTypeIdentifier>>());
		AddOutput(OutParamTextureImage(), MakeUnique<TBasicNodeOutput<FTextureImage, FTextureImage::DataTypeIdentifier>>());
	}

	UE_API virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;
};

}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
