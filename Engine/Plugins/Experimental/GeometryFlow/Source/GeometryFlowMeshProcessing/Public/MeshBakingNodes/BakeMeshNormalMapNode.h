// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "DataTypes/MeshImageBakingData.h"
#include "DataTypes/NormalMapData.h"
#include "DynamicMesh/MeshTangents.h"

#include "BakeMeshNormalMapNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

USTRUCT()
struct FBakeMeshNormalMapSettings 
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::BakeNormalMapSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double MaxDistance = 0.0;
};

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FBakeMeshNormalMapSettings, BakeMeshNormalMap, 1);



class FBakeMeshNormalMapNode : public FNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FBakeMeshNormalMapNode, Version, FNode)

protected:
	using SettingsDataType = TMovableData<FBakeMeshNormalMapSettings, FBakeMeshNormalMapSettings::DataTypeIdentifier>;

public:
	static const FString InParamBakeCache() { return TEXT("BakeCache"); }
	static const FString InParamTangents() { return TEXT("Tangents"); }
	static const FString InParamSettings() { return TEXT("Settings"); }

	static const FString OutParamNormalMap() { return TEXT("NormalMap"); }

public:
	FBakeMeshNormalMapNode()
	{
		AddInput(InParamBakeCache(), MakeUnique<TImmutableNodeInput<FMeshBakingCache, FMeshBakingCache::DataTypeIdentifier>>());
		AddInput(InParamTangents(), MakeUnique<TBasicNodeInput<FMeshTangentsd, (int)EMeshProcessingDataTypes::MeshTangentSet>>());
		AddInput(InParamSettings(), MakeUnique<TBasicNodeInput<FBakeMeshNormalMapSettings, FBakeMeshNormalMapSettings::DataTypeIdentifier>>());

		AddOutput(OutParamNormalMap(), MakeUnique<TBasicNodeOutput<FNormalMapImage, FNormalMapImage::DataTypeIdentifier>>());
	}

	UE_API virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;
};






}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
