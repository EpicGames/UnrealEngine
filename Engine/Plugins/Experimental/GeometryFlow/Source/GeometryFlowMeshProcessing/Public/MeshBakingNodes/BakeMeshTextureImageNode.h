// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowCoreNodes.h"
#include "DataTypes/MeshImageBakingData.h"
#include "DataTypes/TextureImageData.h"

#include "BakeMeshTextureImageNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

USTRUCT()
struct FBakeMeshTextureImageSettings 
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::BakeTextureImageSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 DetailUVLayer = 0;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double MaxDistance = 0.0;
};

namespace UE
{
namespace GeometryFlow
{


GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FBakeMeshTextureImageSettings, BakeMeshTextureImage, 1);



class FBakeMeshTextureImageNode : public FNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FBakeMeshTextureImageNode, Version, FNode)

protected:
	using SettingsDataType = TMovableData<FBakeMeshTextureImageSettings, FBakeMeshTextureImageSettings::DataTypeIdentifier>;

public:
	static const FString InParamBakeCache() { return TEXT("BakeCache"); }
	static const FString InParamImage() { return TEXT("TextureImage"); }
	static const FString InParamSettings() { return TEXT("Settings"); }

	static const FString OutParamTextureImage() { return TEXT("TextureImage"); }

public:
	FBakeMeshTextureImageNode()
	{
		AddInput(InParamBakeCache(), MakeUnique<TImmutableNodeInput<FMeshBakingCache, FMeshBakingCache::DataTypeIdentifier>>());
		AddInput(InParamImage(), MakeUnique<TBasicNodeInput<FTextureImage, FTextureImage::DataTypeIdentifier>>());
		AddInput(InParamSettings(), MakeUnique<TBasicNodeInput<FBakeMeshTextureImageSettings, FBakeMeshTextureImageSettings::DataTypeIdentifier>>());

		AddOutput(OutParamTextureImage(), MakeUnique<TBasicNodeOutput<FTextureImage, FTextureImage::DataTypeIdentifier>>());
	}

	UE_API virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;
};






}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
