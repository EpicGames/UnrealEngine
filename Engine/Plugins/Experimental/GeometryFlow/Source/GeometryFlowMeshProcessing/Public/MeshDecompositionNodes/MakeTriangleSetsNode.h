// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "GeometryFlowMovableData.h"
#include "GeometryFlowNodeUtil.h"
#include "DataTypes/DynamicMeshData.h"
#include "DataTypes/IndexSetsData.h"
#include "DataTypes/WeightMapData.h"
#include "Selections/MeshConnectedComponents.h"

namespace UE
{
namespace GeometryFlow
{


class FMakeTriangleSetsFromMeshNode : public FNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMakeTriangleSetsFromMeshNode, Version, FNode)
public:
	static const FString InParam() { return TEXT("Mesh"); }

	static const FString OutParamIndexSets() { return TEXT("IndexSets"); }

public:
	FMakeTriangleSetsFromMeshNode()
	{
		AddInput(InParam(), MakeUnique<FDynamicMeshInput>());
		
		AddOutput(OutParamIndexSets(), MakeBasicOutput<FIndexSets>());
	}

	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamIndexSets())))
		{
			bool bAllInputsValid = true;
			bool bRecomputeRequired = (IsOutputAvailable(OutParamIndexSets()) == false);
			TSafeSharedPtr<IData> MeshArg = FindAndUpdateInputForEvaluate(InParam(), DatasIn, bRecomputeRequired, bAllInputsValid);
			CheckAdditionalInputs(DatasIn, bRecomputeRequired, bAllInputsValid);

			if (bAllInputsValid)
			{
				if (bRecomputeRequired)
				{
					const FDynamicMesh3& Mesh = MeshArg->GetDataConstRef<FDynamicMesh3>((int)EMeshProcessingDataTypes::DynamicMesh);

					FIndexSets NewSets;
					ComputeIndexSets(DatasIn, Mesh, NewSets);

					SetOutput(OutParamIndexSets(), MakeMovableData<FIndexSets>(MoveTemp(NewSets)));
					EvaluationInfo->CountCompute(this);
				}
				DatasOut.SetData(OutParamIndexSets(), GetOutput(OutParamIndexSets()));
			}
		}
	}


	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid)
	{
		// none
	}


	virtual void ComputeIndexSets(const FNamedDataMap& DatasIn, const FDynamicMesh3& Mesh, FIndexSets& SetsOut)
	{
		SetsOut.IndexSets.SetNum(1);
		SetsOut.IndexSets[0].Reserve(Mesh.TriangleCount());
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			SetsOut.IndexSets[0].Add(tid);
		}
	}

};



///
///  Gather triangles into sets based on their GroupIDs. Optionally exclude any triangles belonging to specified groups.
///
class FMakeTriangleSetsFromGroupsNode : public FMakeTriangleSetsFromMeshNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMakeTriangleSetsFromGroupsNode, Version, FMakeTriangleSetsFromMeshNode)
public:
	static const FString InParamGroupLayer() { return TEXT("GroupLayerName"); }
	static const FString InParamIgnoreGroups() { return TEXT("IgnoreGroups"); }

public:
	FMakeTriangleSetsFromGroupsNode() : FMakeTriangleSetsFromMeshNode()
	{
		AddInput(InParamGroupLayer(), 
				 MakeUnique<TBasicNodeInput<FName, (int)EDataTypes::Name>>(), 
				 MakeSafeShared<TMovableData<FName, (int)EDataTypes::Name>>());

		AddInput(InParamIgnoreGroups(), 
				 MakeBasicInput<FIndexSets>());
	}

	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid) override
	{
		FindAndUpdateInputForEvaluate(InParamGroupLayer(), DatasIn, bRecomputeRequired, bAllInputsValid);
		FindAndUpdateInputForEvaluate(InParamIgnoreGroups(), DatasIn, bRecomputeRequired, bAllInputsValid);
	}

protected:

	void ComputeIndexSetsForGroups(const FDynamicMesh3& Mesh,
								   const TSet<int32>& IgnoreGroups,
								   TFunction<int32(int)> TriangleGroupFn,
								   FIndexSets& SetsOut)
	{
		TMap<int32, int32> GroupsMap;
		TArray<int32> GroupCounts;
		int32 NumGroups = 0;
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			int32 GroupID = TriangleGroupFn(tid);
			if (IgnoreGroups.Contains(GroupID))
			{
				continue;
			}

			int32* FoundIndex = GroupsMap.Find(GroupID);
			if (FoundIndex == nullptr)
			{
				int32 Index = NumGroups++;
				GroupsMap.Add(GroupID, Index);
				GroupCounts.Add(1);
			}
			else
			{
				GroupCounts[*FoundIndex]++;
			}
		}

		SetsOut.IndexSets.SetNum(NumGroups);
		for (int32 k = 0; k < NumGroups; ++k)
		{
			SetsOut.IndexSets[k].Reserve(GroupCounts[k]);
		}

		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			int32 GroupID = TriangleGroupFn(tid);
			if (IgnoreGroups.Contains(GroupID))
			{
				continue;
			}

			int32* FoundIndex = GroupsMap.Find(GroupID);
			SetsOut.IndexSets[*FoundIndex].Add(tid);
		}
	}

public:

	virtual void ComputeIndexSets(const FNamedDataMap& DatasIn, const FDynamicMesh3& Mesh, FIndexSets& SetsOut) override
	{
		TSafeSharedPtr<IData> IgnoreGroupsArg = DatasIn.FindData(InParamIgnoreGroups());
		const FIndexSets& IgnoreGroupsSets = IgnoreGroupsArg->GetDataConstRef<FIndexSets>(FIndexSets::DataTypeIdentifier);
		TSet<int32> IgnoreGroups;
		IgnoreGroupsSets.GetAllValues(IgnoreGroups);

		TSafeSharedPtr<IData> GroupLayerArg = DatasIn.FindData(InParamGroupLayer());
		FName GroupName = GroupLayerArg->GetDataConstRef<FName>((int)EDataTypes::Name);

		if (GroupName.IsNone())
		{
			return;
		}
		if (Mesh.HasTriangleGroups() && (GroupName == "Default"))
		{
			ComputeIndexSetsForGroups(Mesh, IgnoreGroups, [&Mesh](int TriangleID) { return Mesh.GetTriangleGroup(TriangleID); }, SetsOut);
		}
		else if (Mesh.HasAttributes())
		{
			for (int32 PolygroupLayerIndex = 0; PolygroupLayerIndex < Mesh.Attributes()->NumPolygroupLayers(); ++PolygroupLayerIndex)
			{
				const FDynamicMeshPolygroupAttribute* PolygroupLayer = Mesh.Attributes()->GetPolygroupLayer(PolygroupLayerIndex);
				if (PolygroupLayer->GetName() == GroupName)
				{
					ComputeIndexSetsForGroups(Mesh, IgnoreGroups, [&PolygroupLayer](int TriangleID) { return PolygroupLayer->GetValue(TriangleID); }, SetsOut);
				}
			}		
		}

	}

};


class FMakeTriangleSetsFromConnectedComponentsNode : public FMakeTriangleSetsFromMeshNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMakeTriangleSetsFromConnectedComponentsNode, Version, FMakeTriangleSetsFromMeshNode)
public:

	virtual void ComputeIndexSets(const FNamedDataMap& DatasIn, const FDynamicMesh3& Mesh, FIndexSets& SetsOut) override
	{
		FMeshConnectedComponents MeshRegions(&Mesh);
		MeshRegions.FindConnectedTriangles();

		for (const FMeshConnectedComponents::FComponent& Component : MeshRegions.Components)
		{
			SetsOut.AppendSet(Component.Indices);
		}
	}

};


/// If one triangle vertex has a weight greater than the given threshold, include it in the output triangle set.
/// TODO: Optionally make it so the *average* triangle vertex weight has to exceed the threshold. Or the minumum triangle
/// vertex weight.
class FMakeTriangleSetsFromWeightMapNode : public FMakeTriangleSetsFromMeshNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMakeTriangleSetsFromWeightMapNode, Version, FMakeTriangleSetsFromMeshNode)
public:

	static const FString InParamWeightMap() { return TEXT("WeightMap"); }
	static const FString InParamThreshold() { return TEXT("Threshold"); }

	FMakeTriangleSetsFromWeightMapNode() : FMakeTriangleSetsFromMeshNode()
	{
		AddInput(InParamWeightMap(), MakeBasicInput<FWeightMap>());
		AddInput(InParamThreshold(), MakeUnique<TBasicNodeInput<float, (int)EDataTypes::Float>>());
	}

	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid) override
	{
		FindAndUpdateInputForEvaluate(InParamWeightMap(), DatasIn, bRecomputeRequired, bAllInputsValid);
		FindAndUpdateInputForEvaluate(InParamThreshold(), DatasIn, bRecomputeRequired, bAllInputsValid);
	}

	virtual void ComputeIndexSets(const FNamedDataMap& DatasIn, const FDynamicMesh3& Mesh, FIndexSets& SetsOut) override
	{
		TSafeSharedPtr<IData> WeightMapArg = DatasIn.FindData(InParamWeightMap());
		const FWeightMap& WeightMap = WeightMapArg->GetDataConstRef<FWeightMap>(FWeightMap::DataTypeIdentifier);
		const TArray<float>& Weights = WeightMap.Weights;
		check(Weights.Num() >= Mesh.MaxVertexID());

		TSafeSharedPtr<IData> ThresholdArg = DatasIn.FindData(InParamThreshold());
		const float Threshold = ThresholdArg->GetDataConstRef<float>((int)EDataTypes::Float);

		TArray<int32> Set;

		for (int TriangleID : Mesh.TriangleIndicesItr())
		{
			FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
			for (int VertexID : {Triangle[0], Triangle[1], Triangle[2]} )
			{
				if (Weights[VertexID] > Threshold)
				{
					Set.Add(TriangleID);
					break;
				}
			}
		}

		SetsOut.AppendSet(Set);
	}

};


}	// end namespace GeometryFlow
}	// end namespace UE
