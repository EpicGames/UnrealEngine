// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE
{
namespace GeometryFlow
{

	struct FMeshProcessingNodeRegistration
	{
		// Required registration of the geometry flow graph nodes in the MeshProcessing Module.
		GEOMETRYFLOWMESHPROCESSING_API static void RegisterNodes();
	};


}
}