// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"

namespace UE
{
namespace GeometryFlow
{


/**
 * FCompactMeshNode compacts the input mesh, to remove gaps in the vertex/triangle indexing. This node can be applied in-place.
 */
class FCompactMeshNode : public FProcessMeshBaseNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FCompactMeshNode, Version, FProcessMeshBaseNode)

public:
	FCompactMeshNode()
	{
		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut.CompactCopy(MeshIn);
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshInOut.CompactInPlace();
	}

};





}	// end namespace GeometryFlow
}	//
