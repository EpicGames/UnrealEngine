// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingBaseNodes.h"

#include "GenerateConvexHullMeshNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

USTRUCT()
struct FGenerateConvexHullMeshSettings
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::GenerateConvexHullMeshSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bPrefilterVertices = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int PrefilterGridResolution = 10;
};

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

// bring settings into the namespace.  @todo - fix code the expects the UE::GeometryFlow:: for this struct and delete this
typedef FGenerateConvexHullMeshSettings FGenerateConvexHullMeshSettings;

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FGenerateConvexHullMeshSettings, GenerateConvexHullMesh, 1);


class FGenerateConvexHullMeshNode : public TProcessMeshWithSettingsBaseNode<FGenerateConvexHullMeshSettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FGenerateConvexHullMeshNode, Version, FNode)

public:

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FGenerateConvexHullMeshSettings& SettingsIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MakeConvexHullMesh(MeshIn, SettingsIn, MeshOut, EvaluationInfo);
	}

	UE_API EGeometryFlowResult MakeConvexHullMesh(const FDynamicMesh3& MeshIn,
		const FGenerateConvexHullMeshSettings& Settings,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo);

};

}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
