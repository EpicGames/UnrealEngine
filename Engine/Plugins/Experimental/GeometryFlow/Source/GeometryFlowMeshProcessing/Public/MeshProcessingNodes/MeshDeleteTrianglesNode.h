// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowNodeUtil.h"
#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"
#include "DataTypes/IndexSetsData.h"
#include "DynamicMeshEditor.h"

namespace UE
{
namespace GeometryFlow
{


class FMeshDeleteTrianglesNode : public FProcessMeshBaseNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMeshDeleteTrianglesNode, Version, FProcessMeshBaseNode)

public:
	static const FString InParamTriangleLists() { return TEXT("TriangleLists"); }


public:
	FMeshDeleteTrianglesNode()
	{
		AddInput(InParamTriangleLists(), MakeBasicInput<FIndexSets>());

		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}

	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid) override
	{
		FindAndUpdateInputForEvaluate(InParamTriangleLists(), DatasIn, bRecomputeRequired, bAllInputsValid);
	}


	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut = MeshIn;
		DeleteTrianglesForMesh(DatasIn, MeshOut);
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		DeleteTrianglesForMesh(DatasIn, MeshInOut);
	}


	void DeleteTrianglesForMesh(const FNamedDataMap& DatasIn, FDynamicMesh3& EditMesh)
	{
		TSafeSharedPtr<IData> TriangleSetsArg = DatasIn.FindData(InParamTriangleLists());
		const FIndexSets& TriangleSets = TriangleSetsArg->GetDataConstRef<FIndexSets>(FIndexSets::DataTypeIdentifier);

		FDynamicMeshEditor Editor(&EditMesh);
		for (const TArray<int32>& Set : TriangleSets.IndexSets)
		{
			Editor.RemoveTriangles(Set, true);
		}
	}
};





}	// end namespace GeometryFlow
}	//
