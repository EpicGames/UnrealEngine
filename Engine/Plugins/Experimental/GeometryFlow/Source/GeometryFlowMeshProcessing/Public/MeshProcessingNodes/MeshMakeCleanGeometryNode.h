// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"

#include "MeshMakeCleanGeometryNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

USTRUCT()
struct  FMeshMakeCleanGeometrySettings 
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::MakeCleanGeometrySettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int FillHolesEdgeCountThresh = 8;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double FillHolesEstimatedAreaFraction = 0.001;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bDiscardAllAttributes = false;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bClearUVs = true;
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bClearNormals = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bClearTangents = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bClearVertexColors = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bClearMaterialIDs = false;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bOutputMeshVertexNormals = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bOutputOverlayVertexNormals = true;
};

namespace UE
{
namespace GeometryFlow
{

// bringing this into the namespace so other code that relies on UE::GeometryFlow::FMeshMakeCleanGeometrySettings will just work
// @todo, update client code and remove this type def
typedef FMeshMakeCleanGeometrySettings FMeshMakeCleanGeometrySettings;



GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshMakeCleanGeometrySettings, MeshMakeCleanGeometry, 1);


class FMeshMakeCleanGeometryNode : public TProcessMeshWithSettingsBaseNode<FMeshMakeCleanGeometrySettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMeshMakeCleanGeometryNode, Version, FNode)

public:

	UE_API void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshMakeCleanGeometrySettings& SettingsIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;

	UE_API void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const FMeshMakeCleanGeometrySettings& Settings,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;

protected:

	UE_API void ApplyMakeCleanGeometry(FDynamicMesh3& MeshInOut, const FMeshMakeCleanGeometrySettings& Settings);
	
};


}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
