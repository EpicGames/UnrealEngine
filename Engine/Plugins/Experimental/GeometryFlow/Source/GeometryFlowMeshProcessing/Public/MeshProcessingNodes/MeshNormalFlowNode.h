// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingNodes/MeshSimplifyNode.h"
#include "Remesher.h"
#include "GeometryFlowTSourceNode.h"

#include "MeshNormalFlowNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

UENUM()
enum class EGeometryFlow_SmoothTypes
{
	Uniform = 0,	/** Uniform weights, produces regular mesh and fastest convergence */
	Cotan = 1,		/** Cotangent weights prevent tangential flow and hence preserve triangle shape / texture coordinates, but can become unstable... */
	MeanValue = 2   /** Mean Value weights also have reduced tangential flow but are never negative and hence more stable */
};


// conversions
namespace UE
{
namespace GeometryFlow
{
	static UE::Geometry::FRemesher::ESmoothTypes FromUEnum(const EGeometryFlow_SmoothTypes& SmoothType) { return static_cast<FRemesher::ESmoothTypes>(SmoothType); }
	static EGeometryFlow_SmoothTypes ToUEnum(const UE::Geometry::FRemesher::ESmoothTypes& SmoothType) { return static_cast<EGeometryFlow_SmoothTypes>(SmoothType); }
}
}

USTRUCT()
struct FMeshNormalFlowSettings : public FMeshSimplifySettings
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::NormalFlowSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int MaxRemeshIterations = 20;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int NumExtraProjectionIterations = 5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bFlips = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bSplits = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bCollapses = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_SmoothTypes SmoothingType = UE::GeometryFlow::ToUEnum(UE::Geometry::FRemesher::ESmoothTypes::Uniform);
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float SmoothingStrength = 0.25f;
};

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshNormalFlowSettings, MeshNormalFlow, 1);


class FMeshNormalFlowNode : public TProcessMeshWithSettingsBaseNode<FMeshNormalFlowSettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMeshNormalFlowNode, Version, FNode)

public:

	// Beside the intput mesh and settings, we also take a "target mesh" that will be the projection target.
	static const FString InParamTargetMesh() { return TEXT("TargetMesh"); }

	FMeshNormalFlowNode() : TProcessMeshWithSettingsBaseNode<FMeshNormalFlowSettings>()
	{
		AddInput(InParamTargetMesh(), MakeUnique<FDynamicMeshInput>());
	}

	UE_API void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshNormalFlowSettings& SettingsIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;

	UE_API void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const FMeshNormalFlowSettings& SettingsIn,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;

	UE_API void CheckAdditionalInputs(const FNamedDataMap& DatasIn, 
							   bool& bRecomputeRequired, 
							   bool& bAllInputsValid) override;


protected:

	UE_API void DoNormalFlow(const FMeshNormalFlowSettings& SettingsIn,
					  const FDynamicMesh3& ProjectionTargetMesh,
					  bool bAttributesHaveBeenDiscarded,
					  FDynamicMesh3& EditMesh);

};


}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
