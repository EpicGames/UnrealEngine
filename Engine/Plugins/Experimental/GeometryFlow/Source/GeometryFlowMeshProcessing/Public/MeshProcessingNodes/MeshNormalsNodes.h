// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"

#include "DynamicMesh/MeshNormals.h"

#include "MeshNormalsNodes.generated.h"

UENUM()
enum class EGeometryFlow_ComputeNormalsType
{
	PerTriangle = 0,
	PerVertex = 1,
	RecomputeExistingTopology = 2,
	FromFaceAngleThreshold = 3,
	FromGroups = 4
};

USTRUCT()
struct FMeshNormalsSettings 
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::NormalsSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_ComputeNormalsType NormalsType = EGeometryFlow_ComputeNormalsType::FromFaceAngleThreshold;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bInvert = false;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bAreaWeighted = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bAngleWeighted = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow") // for FromAngleThreshold type
	double AngleThresholdDeg = 180.0;
};

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

enum class EComputeNormalsType
{
	PerTriangle = 0,
	PerVertex = 1,
	RecomputeExistingTopology = 2,
	FromFaceAngleThreshold = 3,
	FromGroups = 4
};

static EComputeNormalsType FromUEnum(const EGeometryFlow_ComputeNormalsType& ComputeNormalsType)
{
	return static_cast<EComputeNormalsType>(ComputeNormalsType);
}
static EGeometryFlow_ComputeNormalsType ToUEnum(const EComputeNormalsType& ComputeNormalsType)
{
	return static_cast<EGeometryFlow_ComputeNormalsType>(ComputeNormalsType);
}


// bring settings into the namespace.  @todo - fix code the expects the UE::GeometryFlow:: for this struct and delete this
typedef FMeshNormalsSettings FMeshNormalsSettings;

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshNormalsSettings, Normals, 1);


/**
 * Recompute Normals overlay for input Mesh. Can apply in-place.
 */
class FComputeMeshNormalsNode : public TProcessMeshWithSettingsBaseNode<FMeshNormalsSettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FComputeMeshNormalsNode, Version, FNode)
public:
	FComputeMeshNormalsNode() : TProcessMeshWithSettingsBaseNode<FMeshNormalsSettings>()
	{
		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}


	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshNormalsSettings& Settings,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut = MeshIn;
		ComputeNormals(Settings, MeshOut);
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const FMeshNormalsSettings& Settings,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		ComputeNormals(Settings, MeshInOut);
	}


	void ComputeNormals(const FMeshNormalsSettings& Settings, FDynamicMesh3& MeshInOut)
	{
		if (MeshInOut.HasAttributes() == false)
		{
			MeshInOut.EnableAttributes();
		}
		FDynamicMeshNormalOverlay* Normals = MeshInOut.Attributes()->PrimaryNormals();
		if (Normals == nullptr)
		{
			MeshInOut.Attributes()->SetNumNormalLayers(1);
			Normals = MeshInOut.Attributes()->PrimaryNormals();
		}

		if (Settings.NormalsType == ToUEnum( EComputeNormalsType::PerTriangle))
		{
			ensure(Settings.bInvert == false);		// not supported
			FMeshNormals::InitializeMeshToPerTriangleNormals(&MeshInOut);
			return;
		}
		else if (Settings.NormalsType == ToUEnum( EComputeNormalsType::PerVertex))
		{
			ensure(Settings.bInvert == false);		// not supported
			FMeshNormals::InitializeOverlayToPerVertexNormals(Normals, false);
			return;
		}


		if (Settings.NormalsType == ToUEnum( EComputeNormalsType::FromFaceAngleThreshold))
		{
			FMeshNormals::InitializeOverlayTopologyFromOpeningAngle(&MeshInOut, Normals, Settings.AngleThresholdDeg);
		}
		else if (Settings.NormalsType == ToUEnum( EComputeNormalsType::FromGroups))
		{
			FMeshNormals::InitializeOverlayTopologyFromFaceGroups(&MeshInOut, Normals);
		}
		else if (Settings.NormalsType != ToUEnum( EComputeNormalsType::RecomputeExistingTopology))
		{
			ensure(false);
		}

		FMeshNormals MeshNormals(&MeshInOut);
		MeshNormals.RecomputeOverlayNormals(Normals, Settings.bAreaWeighted, Settings.bAngleWeighted);
		MeshNormals.CopyToOverlay(Normals, Settings.bInvert);
	}

};



/**
 * Recompute per-vertex normals in Normals Overlay for input mesh. Can apply in-place.
 */
class FComputeMeshPerVertexOverlayNormalsNode : public FSimpleInPlaceProcessMeshBaseNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FComputeMeshPerVertexOverlayNormalsNode, Version, FSimpleInPlaceProcessMeshBaseNode)
public:
	virtual void ApplyNodeToMesh(FDynamicMesh3& MeshInOut, TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (MeshInOut.HasAttributes() == false)
		{
			MeshInOut.EnableAttributes();
		}
		FDynamicMeshNormalOverlay* Normals = MeshInOut.Attributes()->PrimaryNormals();
		if (Normals == nullptr)
		{
			MeshInOut.Attributes()->SetNumNormalLayers(1);
			Normals = MeshInOut.Attributes()->PrimaryNormals();
		}
		FMeshNormals::InitializeOverlayToPerVertexNormals(Normals, false);
	}
};



/**
 * Recompute per-vertex normals stored directly on the mesh. Can apply in-place.
 */
class FComputeMeshPerVertexNormalsNode : public FSimpleInPlaceProcessMeshBaseNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FComputeMeshPerVertexNormalsNode, Version, FSimpleInPlaceProcessMeshBaseNode)
public:
	virtual void ApplyNodeToMesh(FDynamicMesh3& MeshInOut, TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		FMeshNormals::QuickComputeVertexNormals(MeshInOut, false);
	}
};




}	// end namespace GeometryFlow
}	// end namespace UE
