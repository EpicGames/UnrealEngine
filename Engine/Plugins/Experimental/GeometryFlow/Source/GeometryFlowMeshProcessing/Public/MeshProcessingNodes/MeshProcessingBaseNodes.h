// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryFlowMeshProcessingModule.h"
#include "DataTypes/DynamicMeshData.h"
#include "GeometryFlowMovableData.h"
#include "Util/ProgressCancel.h"

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;


class FProcessMeshBaseNode : public FNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FProcessMeshBaseNode, Version, FNode)
public:
	static const FString InParamMesh() { return TEXT("Mesh"); }
	static const FString OutParamResultMesh() { return TEXT("ResultMesh"); }

public:
	FProcessMeshBaseNode()
	{
		AddInput(InParamMesh(), MakeUnique<FDynamicMeshInput>());

		AddOutput(OutParamResultMesh(), MakeUnique<FDynamicMeshOutput>());
	}


protected:

	//
	// FProcessMeshBaseNode API that subclasses must/can implement
	//

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		check(false);		// subclass must implement ProcessMesh()
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		ensureMsgf(false, TEXT("TProcessMeshBaseNode::ProcessMeshInPlace called but not defined!"));
	}


	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid)
	{
		// none
	}

public:

	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamResultMesh())))
		{
			bool bAllInputsValid = true;
			bool bRecomputeRequired = ( IsOutputAvailable(OutParamResultMesh()) == false );
			TSafeSharedPtr<IData> MeshArg = FindAndUpdateInputForEvaluate(InParamMesh(), DatasIn, bRecomputeRequired, bAllInputsValid);
			CheckAdditionalInputs(DatasIn, bRecomputeRequired, bAllInputsValid);
			if (bAllInputsValid)
			{
				if (bRecomputeRequired)
				{
					// if execution is interrupted by ProgressCancel, make sure this node is recomputed next time through
					ClearAllOutputs();

					bool bIsMeshMutable = DatasIn.GetDataFlags(InParamMesh()).bIsMutableData;
					if (bIsMeshMutable)
					{
						UE_LOGF(LogGeometryFlowMeshProcessing, Display, "[%ls]  RECOMPUTING MeshOp In Place!", *GetIdentifier());

						FDynamicMesh3 EditableMesh;
						MeshArg->GiveTo<FDynamicMesh3>(EditableMesh, (int)EMeshProcessingDataTypes::DynamicMesh);
						ProcessMeshInPlace(DatasIn, EditableMesh, EvaluationInfo);

						if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
						{
							return;
						}

						// store new result
						TSafeSharedPtr<FDataDynamicMesh> Result = MakeSafeShared<FDataDynamicMesh>();
						Result->MoveData(MoveTemp(EditableMesh));
						SetOutput(OutParamResultMesh(), Result);
					}
					else
					{
						UE_LOGF(LogGeometryFlowMeshProcessing, Display, "[%ls]  RECOMPUTING MeshOp", *GetIdentifier());

						// do we ever want to support using a copy of the source mesh?
						const FDynamicMesh3& SourceMesh = MeshArg->GetDataConstRef<FDynamicMesh3>((int)EMeshProcessingDataTypes::DynamicMesh);

						// run mesh processing
						FDynamicMesh3 ResultMesh;
						ProcessMesh(DatasIn, SourceMesh, ResultMesh, EvaluationInfo);

						if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
						{
							return;
						}

						// store new result
						TSafeSharedPtr<FDataDynamicMesh> Result = MakeSafeShared<FDataDynamicMesh>();
						Result->MoveData(MoveTemp(ResultMesh));
						SetOutput(OutParamResultMesh(), Result);
					}

					EvaluationInfo->CountCompute(this);
				}
				DatasOut.SetData(OutParamResultMesh(), GetOutput(OutParamResultMesh()));
			}
		}
	}
};





/**
* FSimpleInPlaceProcessMeshBaseNode provides a standard pattern for a simple "In-Place" mesh
* processing operation, eg something like recomputing normals where it would not make
* sense to cache the output and there are no configuration settings. The subclass only
* has to implement the ApplyNodeToMesh() function.
*/
class FSimpleInPlaceProcessMeshBaseNode : public FProcessMeshBaseNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FSimpleInPlaceProcessMeshBaseNode, Version, FProcessMeshBaseNode)
public:
	FSimpleInPlaceProcessMeshBaseNode()
	{
		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}

	virtual void ProcessMesh( const FNamedDataMap& DatasIn, const FDynamicMesh3& MeshIn, FDynamicMesh3& MeshOut, TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut = MeshIn;
		ApplyNodeToMesh(MeshOut, EvaluationInfo);
	}

	virtual void ProcessMeshInPlace( const FNamedDataMap& DatasIn, FDynamicMesh3& MeshInOut, TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		ApplyNodeToMesh(MeshInOut, EvaluationInfo);
	}

	// subclasses only have to implement this
	virtual void ApplyNodeToMesh(FDynamicMesh3& MeshInOut, TUniquePtr<FEvaluationInfo>& EvaluationInfo) = 0;

};





namespace Private
{

// This function is a workaround for an issue with Clang targeting Windows.
// A function in a class template cannot contain log statements if that class template
// is derived from by a type annotated with {Module}_API.
inline void LogMeshProcessing(const TCHAR* Identifier, const TCHAR* Message)
{
	UE_LOGF(LogGeometryFlowMeshProcessing, Display, "[%ls]  %ls", Identifier, Message);
}

} // end namespace Private

template<typename SettingsType>
class TProcessMeshWithSettingsBaseNode : public FNode
{
	typedef FNode Super;
public:
	using SettingsDataType = TMovableData<SettingsType, SettingsType::DataTypeIdentifier>;

public:
	static const FString InParamMesh() { return TEXT("Mesh"); }
	static const FString InParamSettings() { return TEXT("Settings"); }
	static const FString OutParamResultMesh() { return TEXT("ResultMesh"); }

protected:
	TProcessMeshWithSettingsBaseNode()
	{
		AddInput(InParamMesh(), MakeUnique<FDynamicMeshInput>());
		AddInput(InParamSettings(), MakeUnique<TBasicNodeInput<SettingsType, SettingsType::DataTypeIdentifier>>());

		AddOutput(OutParamResultMesh(), MakeUnique<FDynamicMeshOutput>());
	}


	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const SettingsType& SettingsIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const SettingsType& SettingsIn,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		ensureMsgf(false, TEXT("TProcessMeshWithSettingsBaseNode::ProcessMeshInPlace called but not defined!"));
	}


	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid)
	{
		// none
	}

public:

	virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		if (ensure(DatasOut.Contains(OutParamResultMesh())))
		{
			bool bAllInputsValid = true;
			bool bRecomputeRequired = ( IsOutputAvailable(OutParamResultMesh()) == false );
			TSafeSharedPtr<IData> MeshArg = FindAndUpdateInputForEvaluate(InParamMesh(), DatasIn, bRecomputeRequired, bAllInputsValid);
			TSafeSharedPtr<IData> SettingsArg = FindAndUpdateInputForEvaluate(InParamSettings(), DatasIn, bRecomputeRequired, bAllInputsValid);
			CheckAdditionalInputs(DatasIn, bRecomputeRequired, bAllInputsValid);
			if (bAllInputsValid)
			{
				if (bRecomputeRequired)
				{
					// if execution is interrupted by ProgressCancel, make sure this node is recomputed next time through
					ClearAllOutputs();

					// always make a copy of settings
					SettingsType Settings;
					SettingsArg->GetDataCopy(Settings, SettingsType::DataTypeIdentifier);

					bool bIsMeshMutable = DatasIn.GetDataFlags(InParamMesh()).bIsMutableData;
					if (bIsMeshMutable)
					{
						Private::LogMeshProcessing(*GetIdentifier(), TEXT("RECOMPUTING MeshOp In Place!"));

						FDynamicMesh3 EditableMesh;
						MeshArg->GiveTo<FDynamicMesh3>(EditableMesh, (int)EMeshProcessingDataTypes::DynamicMesh);
						ProcessMeshInPlace(DatasIn, Settings, EditableMesh, EvaluationInfo);

						if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
						{
							return;
						}

						// store new result
						TSafeSharedPtr<FDataDynamicMesh> Result = MakeSafeShared<FDataDynamicMesh>();
						Result->MoveData(MoveTemp(EditableMesh));
						SetOutput(OutParamResultMesh(), Result);
					}
					else
					{
						Private::LogMeshProcessing(*GetIdentifier(), TEXT("RECOMPUTING MeshOp"));

						// do we ever want to support using a copy of the source mesh?
						const FDynamicMesh3& SourceMesh = MeshArg->GetDataConstRef<FDynamicMesh3>((int)EMeshProcessingDataTypes::DynamicMesh);

						// run mesh processing
						FDynamicMesh3 ResultMesh;
						ProcessMesh(DatasIn, Settings, SourceMesh, ResultMesh, EvaluationInfo);

						if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
						{
							return;
						}

						// store new result
						TSafeSharedPtr<FDataDynamicMesh> Result = MakeSafeShared<FDataDynamicMesh>();
						Result->MoveData(MoveTemp(ResultMesh));
						SetOutput(OutParamResultMesh(), Result);
					}

					EvaluationInfo->CountCompute(this);
				}
				DatasOut.SetData(OutParamResultMesh(), GetOutput(OutParamResultMesh()));
			}
		}
	}
};



















}	// end namespace GeometryFlow
}	// end namespace UE
