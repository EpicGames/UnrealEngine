// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowTypes.h"

namespace UE
{
namespace GeometryFlow
{

enum class EMeshProcessingDataTypes
{
	DynamicMesh = (int)EDataTypes::BaseMeshProcessingTypes + 1,
	MeshNormalSet = (int)EDataTypes::BaseMeshProcessingTypes + 2,
	MeshTangentSet = (int)EDataTypes::BaseMeshProcessingTypes + 3,

	BakingCache = (int)EDataTypes::BaseMeshProcessingTypes + 10,
	TextureImage = (int)EDataTypes::BaseMeshProcessingTypes + 11,
	NormalMapImage = (int)EDataTypes::BaseMeshProcessingTypes + 12,
	CollisionGeometry = (int)EDataTypes::BaseMeshProcessingTypes + 13,
	IndexSets = (int)EDataTypes::BaseMeshProcessingTypes + 14,
	WeightMap = (int)EDataTypes::BaseMeshProcessingTypes + 15,
	MaterialIDToTextureMap = (int)EDataTypes::BaseMeshProcessingTypes + 16,

	SolidifySettings = (int)EDataTypes::BaseMeshProcessingTypes + 100,
	VoxMorphologyOpSettings = (int)EDataTypes::BaseMeshProcessingTypes + 101,
	ThickenSettings = (int)EDataTypes::BaseMeshProcessingTypes + 102,
	GenerateConvexHullMeshSettings = (int)EDataTypes::BaseMeshProcessingTypes + 103,

	SimplifySettings = (int)EDataTypes::BaseMeshProcessingTypes + 110,
	NormalFlowSettings = (int)EDataTypes::BaseMeshProcessingTypes + 111,



	NormalsSettings = (int)EDataTypes::BaseMeshProcessingTypes + 120,
	TangentsSettings = (int)EDataTypes::BaseMeshProcessingTypes + 121,

	MakeCleanGeometrySettings = (int)EDataTypes::BaseMeshProcessingTypes + 130,

	RecalculateUVsSettings = (int)EDataTypes::BaseMeshProcessingTypes + 150,
	RepackUVsSettings = (int)EDataTypes::BaseMeshProcessingTypes + 151,


	MakeBakingCacheSettings = (int)EDataTypes::BaseMeshProcessingTypes + 160,
	BakeNormalMapSettings = (int)EDataTypes::BaseMeshProcessingTypes + 161,
	BakeTextureImageSettings = (int)EDataTypes::BaseMeshProcessingTypes + 162,
	BakeMultiTextureSettings = (int)EDataTypes::BaseMeshProcessingTypes + 163,


	GenerateCollisionConvexHullsSettings = (int)EDataTypes::BaseMeshProcessingTypes + 200,
	GenerateSimpleCollisionSettings = (int)EDataTypes::BaseMeshProcessingTypes + 201,
};



}	// end namespace GeometryFlow
}	// end namespace UE
