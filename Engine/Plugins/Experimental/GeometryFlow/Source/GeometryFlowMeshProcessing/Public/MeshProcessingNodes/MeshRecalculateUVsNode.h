// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"

#include "MeshRecalculateUVsNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

UENUM()
enum class EGeometryFlow_RecalculateUVsUnwrapType : uint8
{
	Auto = 0,
	ExpMap = 1,
	Conformal = 2
};

USTRUCT()
struct FMeshRecalculateUVsSettings 
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::RecalculateUVsSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_RecalculateUVsUnwrapType UnwrapType = EGeometryFlow_RecalculateUVsUnwrapType::Auto;
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 UVLayer = 0;
};
namespace UE
{
namespace GeometryFlow
{

enum class ERecalculateUVsUnwrapType : uint8
{
	Auto = 0,
	ExpMap = 1,
	Conformal = 2
};

static ERecalculateUVsUnwrapType FromUEnum(const EGeometryFlow_RecalculateUVsUnwrapType& UVsUnwrapType)
{
	return static_cast<ERecalculateUVsUnwrapType>(UVsUnwrapType);
}
static EGeometryFlow_RecalculateUVsUnwrapType ToUEnum(const ERecalculateUVsUnwrapType& UVsUnwrapType)
{
	return static_cast<EGeometryFlow_RecalculateUVsUnwrapType>(UVsUnwrapType);
}


// bringing this into the namespace so other code that relies on UE::GeometryFlow::FMeshRecalculateUVsSettings will just work
// @todo, update client code and remove this type def
typedef FMeshRecalculateUVsSettings FMeshRecalculateUVsSettings;

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshRecalculateUVsSettings, MeshRecalculateUVs, 1);



class FMeshRecalculateUVsNode : public TProcessMeshWithSettingsBaseNode<FMeshRecalculateUVsSettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMeshRecalculateUVsNode, Version, FNode)
public:
	FMeshRecalculateUVsNode() : TProcessMeshWithSettingsBaseNode<FMeshRecalculateUVsSettings>()
	{
		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshRecalculateUVsSettings& Settings,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut = MeshIn;
		RecalculateUVsOnMesh(MeshOut, Settings);
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const FMeshRecalculateUVsSettings& Settings,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		RecalculateUVsOnMesh(MeshInOut, Settings);
	}

	UE_API void RecalculateUVsOnMesh(FDynamicMesh3& EditMesh, const FMeshRecalculateUVsSettings& Settings);
};





}	// end namespace GeometryFlow
}	// end 

#undef UE_API
