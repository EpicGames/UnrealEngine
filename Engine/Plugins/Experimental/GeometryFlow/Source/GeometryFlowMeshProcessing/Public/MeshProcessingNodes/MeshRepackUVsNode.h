// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"

#include "MeshRepackUVsNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

USTRUCT()
struct FMeshRepackUVsSettings
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::RepackUVsSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 UVLayer = 0;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 TextureResolution = 512;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 GutterSize = 1;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bAllowFlips = false;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FVector2f UVScale = FVector2f::One();

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FVector2f UVTranslation = FVector2f::Zero();
};

namespace UE
{
namespace GeometryFlow
{


GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshRepackUVsSettings, MeshRepackUVs, 1);

class FMeshRepackUVsNode : public TProcessMeshWithSettingsBaseNode<FMeshRepackUVsSettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMeshRepackUVsNode, Version, FNode)
public:
	FMeshRepackUVsNode() : TProcessMeshWithSettingsBaseNode<FMeshRepackUVsSettings>()
	{
		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshRepackUVsSettings& Settings,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut = MeshIn;
		RepackUVsForMesh(MeshOut, Settings);
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const FMeshRepackUVsSettings& Settings,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		RepackUVsForMesh(MeshInOut, Settings);
	}

	UE_API void RepackUVsForMesh(FDynamicMesh3& EditMesh, const FMeshRepackUVsSettings& Settings);
};





}	// end namespace GeometryFlow
}	// end 

#undef UE_API
