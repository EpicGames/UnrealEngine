// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"

#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "ProjectionTargets.h"
#include "Spatial/MeshAABBTree3.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "MeshSimplifyNode.generated.h"



UENUM()
enum class EGeometryFlow_MeshSimplifyType
{
	Standard = 0,
	VolumePreserving = 1,
	AttributeAware = 2
};

UENUM()
enum class EGeomtryFlow_MeshSimplifyTargetType
{
	TriangleCount = 0,
	VertexCount = 1,
	TrianglePercentage = 2,
	GeometricDeviation = 3
};

UENUM()
enum class EGeometryFlow_EdgeRefineFlags
{
	/** Edge is unconstrained */
	NoConstraint = 0,
	/** Edge cannot be flipped */
	NoFlip = 1,
	/** Edge cannot be split */
	NoSplit = 2,
	/** Edge cannot be collapsed */
	NoCollapse = 4,
	/** Edge cannot be flipped, split, or collapsed */
	FullyConstrained = NoFlip | NoSplit | NoCollapse,
	/** Edge can only be split */
	SplitsOnly = NoFlip | NoCollapse,
	/** Edge can only flip*/
	FlipOnly = NoSplit | NoCollapse,
	/** Edge can only collapse*/
	CollapseOnly = NoFlip | NoSplit
};

namespace UE {
namespace GeometryFlow {
static EEdgeRefineFlags ToEdgeRefineFlags2(const EGeometryFlow_EdgeRefineFlags& Flag)
{
	return static_cast<EEdgeRefineFlags>(Flag);
}
} }

USTRUCT()
struct FMeshSimplifySettings 
{
	GENERATED_USTRUCT_BODY()

	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::SimplifySettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_MeshSimplifyType SimplifyType = EGeometryFlow_MeshSimplifyType::AttributeAware;
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeomtryFlow_MeshSimplifyTargetType TargetType = EGeomtryFlow_MeshSimplifyTargetType::TrianglePercentage;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int TargetCount = 100;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float TargetFraction = 0.5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float GeometricTolerance = 0.5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bDiscardAttributes = false;
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bPreventNormalFlips = true;
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bPreserveSharpEdges = false;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bAllowSeamCollapse = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bAllowSeamSplits = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_EdgeRefineFlags MeshBoundaryConstraints = EGeometryFlow_EdgeRefineFlags::NoConstraint;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_EdgeRefineFlags GroupBorderConstraints = EGeometryFlow_EdgeRefineFlags::NoConstraint;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_EdgeRefineFlags MaterialBorderConstraints = EGeometryFlow_EdgeRefineFlags::NoConstraint;
};

namespace UE { namespace GeometryFlow {
GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshSimplifySettings, Simplify, 1);
}}

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

// bringing this into the namespace so other code that relies on UE::GeometryFlow::FMeshSimplifySettings will just work
// @todo, update client code and remove this type def
typedef FMeshSimplifySettings FMeshSimplifySettings;

enum class EMeshSimplifyType
{
	Standard = 0,
	VolumePreserving = 1,
	AttributeAware = 2
};

enum class EMeshSimplifyTargetType
{
	TriangleCount = 0,
	VertexCount = 1,
	TrianglePercentage = 2,
	GeometricDeviation = 3
};



static EEdgeRefineFlags FromUEnum(const EGeometryFlow_EdgeRefineFlags& Flag) { return static_cast<EEdgeRefineFlags>(Flag); }
static EGeometryFlow_EdgeRefineFlags ToUEnum(const EEdgeRefineFlags& Flag) {return static_cast<EGeometryFlow_EdgeRefineFlags>(Flag); }

static EMeshSimplifyType FromUEnum(const EGeometryFlow_MeshSimplifyType& SimplifyType) { return static_cast<EMeshSimplifyType>(SimplifyType);}
static EGeometryFlow_MeshSimplifyType ToUEnum(const EMeshSimplifyType& SimplifyType) { return static_cast<EGeometryFlow_MeshSimplifyType>(SimplifyType); }

static EMeshSimplifyTargetType FromUEnum(const EGeomtryFlow_MeshSimplifyTargetType& TargetType) { return static_cast<EMeshSimplifyTargetType>(TargetType); }
static EGeomtryFlow_MeshSimplifyTargetType ToUEnum(const EMeshSimplifyTargetType& TargetType) { return static_cast<EGeomtryFlow_MeshSimplifyTargetType>(TargetType); }

class FSimplifyMeshNode : public TProcessMeshWithSettingsBaseNode<FMeshSimplifySettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FSimplifyMeshNode, Version, FNode)
public:
	FSimplifyMeshNode() : TProcessMeshWithSettingsBaseNode<FMeshSimplifySettings>()
	{
		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshSimplifySettings& Settings,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut.Copy(MeshIn, true, true, true, !Settings.bDiscardAttributes);
		ApplySimplify(Settings, MeshOut, EvaluationInfo);
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const FMeshSimplifySettings& Settings,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		ApplySimplify(Settings, MeshInOut, EvaluationInfo);
	}


	template<typename SimplifierType> 
	void DoSimplifyOfType(const FMeshSimplifySettings& Settings, FDynamicMesh3* TargetMesh, TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		SimplifierType Simplifier(TargetMesh);

		if (EvaluationInfo && EvaluationInfo->Progress)
		{
			Simplifier.Progress = EvaluationInfo->Progress;
		}

		Simplifier.ProjectionMode = SimplifierType::ETargetProjectionMode::NoProjection;
		Simplifier.DEBUG_CHECK_LEVEL = 0;

		Simplifier.bAllowSeamCollapse = Settings.bAllowSeamCollapse;
		if (Simplifier.bAllowSeamCollapse)
		{
			Simplifier.SetEdgeFlipTolerance(1.e-5);

			// eliminate any bowties in attribute layers
			if (TargetMesh->Attributes())
			{
				TargetMesh->Attributes()->SplitAllBowties();
			}
		}

		FMeshConstraints Constraints;
		FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, *TargetMesh,
			FromUEnum(Settings.MeshBoundaryConstraints),
			FromUEnum(Settings.GroupBorderConstraints),
			FromUEnum(Settings.MaterialBorderConstraints),
			Settings.bAllowSeamSplits, !Settings.bPreserveSharpEdges, Settings.bAllowSeamCollapse);
		Simplifier.SetExternalConstraints(MoveTemp(Constraints));

		if (Settings.TargetType == EGeomtryFlow_MeshSimplifyTargetType::TrianglePercentage)
		{
			int32 UseTarget = FMath::Max(4, (int)(Settings.TargetFraction * (double)TargetMesh->TriangleCount()));
			Simplifier.SimplifyToTriangleCount(UseTarget);
		}
		else if (Settings.TargetType == EGeomtryFlow_MeshSimplifyTargetType::TriangleCount)
		{
			Simplifier.SimplifyToTriangleCount( FMath::Max(1,Settings.TargetCount) );
		}
		else if (Settings.TargetType == EGeomtryFlow_MeshSimplifyTargetType::VertexCount)
		{
			Simplifier.SimplifyToVertexCount( FMath::Max(3, Settings.TargetCount) );
		}
		else if (Settings.TargetType == EGeomtryFlow_MeshSimplifyTargetType::GeometricDeviation)
		{
			// need to create projection target to measure error
			FDynamicMesh3 MeshCopy;
			MeshCopy.Copy(*TargetMesh, false, false, false, false);
			FDynamicMeshAABBTree3 MeshCopySpatial(&MeshCopy, true);
			FMeshProjectionTarget ProjTarget(&MeshCopy, &MeshCopySpatial);
			Simplifier.SetProjectionTarget(&ProjTarget);

			Simplifier.GeometricErrorConstraint = SimplifierType::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
			Simplifier.GeometricErrorTolerance = Settings.GeometricTolerance;
			Simplifier.SimplifyToVertexCount(3);
		}
		else
		{
			check(false);
		}
	}


	void ApplySimplify(const FMeshSimplifySettings& Settings, FDynamicMesh3& MeshInOut, TUniquePtr<FEvaluationInfo>& EvaluationInfo)
	{
		if (Settings.bDiscardAttributes)
		{
			MeshInOut.DiscardAttributes();
		}

		if (Settings.SimplifyType == EGeometryFlow_MeshSimplifyType::Standard)
		{
			DoSimplifyOfType<FQEMSimplification>(Settings, &MeshInOut, EvaluationInfo);
		}
		else if (Settings.SimplifyType == EGeometryFlow_MeshSimplifyType::VolumePreserving)
		{
			DoSimplifyOfType<FVolPresMeshSimplification>(Settings, &MeshInOut, EvaluationInfo);
		}
		else if (Settings.SimplifyType == EGeometryFlow_MeshSimplifyType::AttributeAware)
		{
			DoSimplifyOfType<FAttrMeshSimplification>(Settings, &MeshInOut, EvaluationInfo);
		}
		else
		{
			check(false);
		}
	}

};



}	// end namespace GeometryFlow
}	// end namespace UE
