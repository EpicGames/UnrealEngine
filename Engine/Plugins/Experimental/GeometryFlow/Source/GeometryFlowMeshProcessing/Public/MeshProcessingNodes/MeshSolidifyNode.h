// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Generators/MeshShapeGenerator.h"
#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Implicit/Solidify.h"
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"

#include "MeshSolidifyNode.generated.h"

USTRUCT()
struct FMeshSolidifySettings
{
	GENERATED_USTRUCT_BODY()

	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::SolidifySettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int VoxelResolution = 64;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float WindingThreshold = 0.5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int SurfaceConvergeSteps = 5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float ExtendBounds = 2.0f;
};



namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;


// bringing this into the namespace so other code that relies on UE::GeometryFlow::FMeshSolidifySettings will just work
// @todo, update client code and remove this type def
typedef FMeshSolidifySettings FMeshSolidifySettings;

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshSolidifySettings, Solidify, 1);


class FSolidifyMeshNode : public TProcessMeshWithSettingsBaseNode<FMeshSolidifySettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FSolidifyMeshNode, Version, FNode)
public:

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshSolidifySettings& SettingsIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		ApplySolidify(MeshIn, SettingsIn, MeshOut);
	}

	void ApplySolidify(const FDynamicMesh3& MeshIn, const FMeshSolidifySettings& Settings, FDynamicMesh3& MeshOut)
	{
		FAxisAlignedBox3d Bounds = MeshIn.GetBounds();

		FDynamicMeshAABBTree3 MeshBVTree(&MeshIn);
		TFastWindingTree<FDynamicMesh3> FastWinding(&MeshBVTree);

		TImplicitSolidify<FDynamicMesh3> SolidifyCalc(&MeshIn, &MeshBVTree, &FastWinding);

		SolidifyCalc.SetCellSizeAndExtendBounds(Bounds, Settings.ExtendBounds, Settings.VoxelResolution);
		SolidifyCalc.WindingThreshold = Settings.WindingThreshold;
		SolidifyCalc.SurfaceSearchSteps = Settings.SurfaceConvergeSteps;
		SolidifyCalc.bSolidAtBoundaries = true;
		SolidifyCalc.ExtendBounds = Settings.ExtendBounds;

		MeshOut.Copy(&SolidifyCalc.Generate());
	}
};


}	// end namespace GeometryFlow
}	// end namespace UE
