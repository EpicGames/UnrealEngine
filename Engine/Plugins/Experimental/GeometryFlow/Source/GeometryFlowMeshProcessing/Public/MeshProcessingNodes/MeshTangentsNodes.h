// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseNodes/TransformerWithSettingsNode.h"
#include "BaseNodes/TransferNode.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshProcessingNodes/MeshProcessingDataTypes.h"

#include "DynamicMesh/MeshTangents.h"
#include "GeometryFlowCoreNodes.h"
#include "GeometryFlowMovableData.h"

#include "MeshTangentsNodes.generated.h"

namespace UE::GeometryFlow { template <typename T, int StorageTypeIdentifier> class TTransferNode; }

UENUM()
enum class EGeometryFlow_ComputeTangentsType
{
	PerTriangle = 0,
	FastMikkT = 1
};

USTRUCT()
struct FMeshTangentsSettings
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::TangentsSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_ComputeTangentsType TangentsType = EGeometryFlow_ComputeTangentsType::FastMikkT;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int UVLayer = 0;
};

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

enum class EComputeTangentsType
{
	PerTriangle = 0,
	FastMikkT = 1
};

static EComputeTangentsType FromUEnum(const EGeometryFlow_ComputeTangentsType& type) { return static_cast<EComputeTangentsType>(type); }
static EGeometryFlow_ComputeTangentsType ToUEnum(const EComputeTangentsType& type) { return static_cast<EGeometryFlow_ComputeTangentsType>(type); }

// bringing this into the namespace so other code that relies on UE::GeometryFlow:: will work
// @todo, update client code and remove this type def
typedef FMeshTangentsSettings FMeshTangentsSettings;


GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshTangentsSettings, Tangents, 1);


class FMeshTangentsTransferNode : public  TTransferNode<FMeshTangentsd, (int)EMeshProcessingDataTypes::MeshTangentSet> {static constexpr int Version =1; GEOMETRYFLOW_NODE_INTERNAL(FMeshTangentsTransferNode, Version, FNode) };


class FComputeMeshTangentsNode : public TTransformerWithSettingsNode<
	FDynamicMesh3, (int)EMeshProcessingDataTypes::DynamicMesh,
	FMeshTangentsSettings, (int)EMeshProcessingDataTypes::TangentsSettings,
	FMeshTangentsd, (int)EMeshProcessingDataTypes::MeshTangentSet>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FComputeMeshTangentsNode, Version, FNode)
public:
	FComputeMeshTangentsNode() : TTransformerWithSettingsNode()
	{
	}

	virtual void ComputeOutput(
		const FNamedDataMap& DatasIn,
		const FMeshTangentsSettings& Settings,
		const FDynamicMesh3& MeshIn,
		FMeshTangentsd& TangentsOut)
	{
		if (ensure(MeshIn.HasAttributes()) == false) return;

		// handle UV layer issues, missing attributes, etc
		int UseUVLayer = Settings.UVLayer;
		bool bValidUVLayer = (UseUVLayer >= 0 && UseUVLayer < MeshIn.Attributes()->NumUVLayers());
		if (ensure(bValidUVLayer) == false)
		{
			UseUVLayer = FMath::Clamp(UseUVLayer, 0, MeshIn.Attributes()->NumUVLayers() - 1);
		}

		FComputeTangentsOptions Options;
		Options.bAveraged = (Settings.TangentsType == ToUEnum( EComputeTangentsType::FastMikkT) );

		TangentsOut.SetMesh(&MeshIn);
		TangentsOut.ComputeTriVertexTangents(
			MeshIn.Attributes()->PrimaryNormals(),
			MeshIn.Attributes()->GetUVLayer(UseUVLayer),
			Options);

		// clear output mesh reference
		TangentsOut.SetMesh(nullptr);
	}


};





}	// end namespace GeometryFlow
}	// end namespace UE
