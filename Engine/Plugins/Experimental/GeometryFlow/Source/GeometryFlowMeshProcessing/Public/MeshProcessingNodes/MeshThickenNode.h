// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"
#include "DataTypes/WeightMapData.h"

#include "MeshThickenNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

USTRUCT()
struct FMeshThickenSettings 
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::ThickenSettings);

	float ThickenAmount = 1.0f;
};

namespace UE
{
namespace GeometryFlow
{

// @todo - remove this when other code is no longer expecting FMeshThickenSettings to be in the UE::GeometryFlow namespace
typedef FMeshThickenSettings FMeshThickenSettings;

GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshThickenSettings, Thicken, 1);


class FMeshThickenNode : public TProcessMeshWithSettingsBaseNode<FMeshThickenSettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMeshThickenNode, Version, FNode)
public:

	static const FString InParamWeightMap() { return TEXT("WeightMap"); }

	FMeshThickenNode()
	{
		AddInput(InParamWeightMap(), MakeUnique<FWeightMapInput>());
	}

	void CheckAdditionalInputs(const FNamedDataMap& DatasIn,
							   bool& bRecomputeRequired,
							   bool& bAllInputsValid) override
	{
		FindAndUpdateInputForEvaluate(InParamWeightMap(), DatasIn, bRecomputeRequired, bAllInputsValid);
	}

	UE_API void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshThickenSettings& SettingsIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;

	UE_API void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		const FMeshThickenSettings& Settings,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;

protected:

	UE_API void ApplyThicken(FDynamicMesh3& MeshInOut, const FMeshThickenSettings& Settings, const TArray<float>& VertexWeights);
	
};


}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
