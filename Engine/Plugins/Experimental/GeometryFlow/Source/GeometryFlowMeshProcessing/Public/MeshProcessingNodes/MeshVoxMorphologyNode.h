// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Generators/MeshShapeGenerator.h"
#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Implicit/Morphology.h"

#include "MeshVoxMorphologyNode.generated.h"

USTRUCT()
struct FVoxMorphologyOpSettings 
{
	GENERATED_USTRUCT_BODY()

	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::VoxMorphologyOpSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int VoxelResolution = 64;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double Distance = 1.0;
};

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

template<> 
struct TSerializationMethod<FVoxMorphologyOpSettings> {static void Serialize(FArchive& Ar, FVoxMorphologyOpSettings& Data) { UStructSerializer(Ar, Data); } };


typedef FVoxMorphologyOpSettings FVoxOffsetSettings;
typedef FVoxMorphologyOpSettings FVoxClosureSettings;
typedef FVoxMorphologyOpSettings FVoxOpeningSettings;


#define GEOMETRYFLOW_DECLARE_MORPHOLOGY_SETTINGS_TYPES(CppType, ReadableName) \
typedef TMovableData<CppType, CppType::DataTypeIdentifier> FData##ReadableName##Settings; \
class F##ReadableName##SettingsSourceNode : public TSourceNodeBase<CppType, CppType::DataTypeIdentifier>{ static constexpr int Version = 1; GEOMETRYFLOW_NODE_INTERNAL(F##ReadableName##SettingsSourceNode, Version, FSourceNodeBase) }; \


GEOMETRYFLOW_DECLARE_MORPHOLOGY_SETTINGS_TYPES(FVoxOffsetSettings, VoxOffset);
GEOMETRYFLOW_DECLARE_MORPHOLOGY_SETTINGS_TYPES(FVoxClosureSettings, VoxClosure);
GEOMETRYFLOW_DECLARE_MORPHOLOGY_SETTINGS_TYPES(FVoxOpeningSettings, VoxOpening);

#undef GEOMETRYFLOW_DECLARE_MORPHOLOGY_SETTINGS_TYPES


template<TImplicitMorphology<FDynamicMesh3>::EMorphologyOp MorphologyOp>
class TVoxMorphologyMeshNode : public TProcessMeshWithSettingsBaseNode<FVoxMorphologyOpSettings>
{
public:


	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FVoxMorphologyOpSettings& SettingsIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		ApplyMorphology(MeshIn, SettingsIn, MeshOut);
	}

	void ApplyMorphology(const FDynamicMesh3& MeshIn, const FVoxMorphologyOpSettings& Settings, FDynamicMesh3& MeshOut)
	{
		if (Settings.Distance == 0.0f)
		{
			MeshOut = MeshIn;
			return;
		}

		FAxisAlignedBox3d Bounds = MeshIn.GetBounds();
		FDynamicMeshAABBTree3 MeshBVTree(&MeshIn);

		TImplicitMorphology<FDynamicMesh3> ImplicitMorphology;
		if (MorphologyOp == TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Dilate && Settings.Distance < 0)
		{
			ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Contract;
		}
		else
		{
			ImplicitMorphology.MorphologyOp = MorphologyOp;
		}
		ImplicitMorphology.Source = &MeshIn;
		ImplicitMorphology.SourceSpatial = &MeshBVTree;
		ImplicitMorphology.SetCellSizesAndDistance(Bounds, Settings.Distance, Settings.VoxelResolution, Settings.VoxelResolution);

		MeshOut.Copy(&ImplicitMorphology.Generate());
	}

};

class FVoxDilateMeshNode : public TVoxMorphologyMeshNode<TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Dilate> { static constexpr int Version = 1; GEOMETRYFLOW_NODE_INTERNAL(FVoxDilateMeshNode, Version, FNode) };
class FVoxClosureMeshNode: public TVoxMorphologyMeshNode<TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Close> { static constexpr int Version = 1;  GEOMETRYFLOW_NODE_INTERNAL(FVoxClosureMeshNode, Version, FNode) };
class FVoxOpeningMeshNode: public  TVoxMorphologyMeshNode<TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Open> { static constexpr int Version = 1;  GEOMETRYFLOW_NODE_INTERNAL(FVoxOpeningMeshNode, Version, FNode) };


}	// end namespace GeometryFlow
}	// end namespace UE
