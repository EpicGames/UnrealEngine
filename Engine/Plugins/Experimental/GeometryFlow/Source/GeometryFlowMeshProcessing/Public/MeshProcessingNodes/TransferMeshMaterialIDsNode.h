// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"
#include "Operations/MeshAttributeTransfer.h"

namespace UE
{
namespace GeometryFlow
{

/**
 * FTransferMeshMaterialIDsNode transfers Material IDs from a SourceMesh to the Input mesh and returns in the Output mesh (can be applied in-place)
 */
class FTransferMeshMaterialIDsNode : public FProcessMeshBaseNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FTransferMeshMaterialIDsNode, Version, FProcessMeshBaseNode)
public:
	static const FString InParamMaterialSourceMesh() { return TEXT("MaterialSourceMesh"); }

public:
	FTransferMeshMaterialIDsNode()
	{
		AddInput(InParamMaterialSourceMesh(), MakeUnique<FDynamicMeshInput>());

		// we can mutate input mesh
		ConfigureInputFlags(InParamMesh(), FNodeInputFlags::Transformable());
	}

	virtual void CheckAdditionalInputs(const FNamedDataMap& DatasIn, bool& bRecomputeRequired, bool& bAllInputsValid) override
	{
		FindAndUpdateInputForEvaluate(InParamMaterialSourceMesh(), DatasIn, bRecomputeRequired, bAllInputsValid);
	}


	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		MeshOut = MeshIn;
		TransferMaterialIDs(DatasIn, &MeshOut);
	}

	virtual void ProcessMeshInPlace(
		const FNamedDataMap& DatasIn,
		FDynamicMesh3& MeshInOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		TransferMaterialIDs(DatasIn, &MeshInOut);
	}


	virtual void TransferMaterialIDs(const FNamedDataMap& DatasIn, FDynamicMesh3* TargetMesh)
	{
		TSafeSharedPtr<IData> MaterialSourceMeshArg = DatasIn.FindData(InParamMaterialSourceMesh());
		const FDynamicMesh3& MaterialSourceMesh = MaterialSourceMeshArg->GetDataConstRef<FDynamicMesh3>((int)EMeshProcessingDataTypes::DynamicMesh);

		if (TargetMesh->HasAttributes() == false)
		{
			TargetMesh->EnableAttributes();
		}
		if (TargetMesh->Attributes()->HasMaterialID() == false)
		{
			TargetMesh->Attributes()->EnableMaterialID();
		}

		FMeshAttributeTransfer Transfer(&MaterialSourceMesh, TargetMesh);
		Transfer.TransferType = EMeshAttributeTransferType::MaterialID;
		Transfer.Apply();
	}

};





}	// end namespace GeometryFlow
}	//
