// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowMovableData.h"
#include "GeometryFlowNodeUtil.h"
#include "ShapeApproximation/MeshSimpleShapeApproximation.h"
#include "DataTypes/DynamicMeshData.h"
#include "DataTypes/CollisionGeometryData.h"
#include "DataTypes/IndexSetsData.h"


#include "GenerateSimpleCollisionNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSING_API

UENUM()
enum class EGeometryFlow_SimpleCollisionGeometryType : uint8
{
	// NOTE: This must be kept in sync with EGenerateStaticMeshLODSimpleCollisionGeometryType in GenerateStaticMeshLODProcess.h

	AlignedBoxes,
	OrientedBoxes,
	MinimalSpheres,
	Capsules,
	ConvexHulls,
	SweptHulls,
	MinVolume,
	None
};

UENUM()
enum class EGeometryFlow_ProjectedHullAxisMode
{
	/** Use Unit X axis */
	X = 0,
	/** Use Unit Y axis */
	Y = 1,
	/** Use Unit Z axis */
	Z = 2,
	/** Use X/Y/Z axis with smallest axis-aligned-bounding-box dimension */
	SmallestBoxDimension = 3,
	/** Compute projected hull for each of X/Y/Z axes and use the one that has the smallest volume  */
	SmallestVolume = 4
};


USTRUCT()
struct FGenerateConvexHullSettings
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int32 SimplifyToTriangleCount = 50;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double GeometricToleranceAfterTarget = 0.;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bPrefilterVertices = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int PrefilterGridResolution = 10;
};

USTRUCT()
struct FGenerateSweptHullSettings
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bSimplifyPolygons = true;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_ProjectedHullAxisMode SweepAxis = EGeometryFlow_ProjectedHullAxisMode::SmallestVolume;
	
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	float HullTolerance = 0.1f;
};

USTRUCT()
struct FGenerateSimpleCollisionSettings
{
	GENERATED_USTRUCT_BODY()
	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypes::GenerateSimpleCollisionSettings);

	EGeometryFlow_SimpleCollisionGeometryType Type = EGeometryFlow_SimpleCollisionGeometryType::ConvexHulls;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FGenerateConvexHullSettings ConvexHullSettings;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	FGenerateSweptHullSettings SweptHullSettings;
};

namespace UE
{
namespace GeometryFlow
{

using namespace UE::Geometry;

typedef FGenerateSimpleCollisionSettings FGenerateSimpleCollisionSettings;


GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FGenerateSimpleCollisionSettings, GenerateSimpleCollision, 1);


class FGenerateSimpleCollisionNode : public FNode
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FGenerateSimpleCollisionNode, Version, FNode)
protected:
	using SettingsDataType = TMovableData<FGenerateSimpleCollisionSettings, FGenerateSimpleCollisionSettings::DataTypeIdentifier>;

public:
	static const FString InParamMesh() { return TEXT("Mesh"); }
	static const FString InParamIndexSets() { return TEXT("TriangleSets"); }	
	static const FString InParamSettings() { return TEXT("Settings"); }
	static const FString OutParamGeometry() { return TEXT("Geometry"); }

public:

	FGenerateSimpleCollisionNode()
	{
		AddInput(InParamMesh(), MakeUnique<FDynamicMeshInput>());
		AddInput(InParamIndexSets(), MakeBasicInput<FIndexSets>());
		AddInput(InParamSettings(), MakeBasicInput<FGenerateSimpleCollisionSettings>());

		AddOutput(OutParamGeometry(), MakeBasicOutput<FCollisionGeometry>());
	}

	UE_API virtual void Evaluate(
		const FNamedDataMap& DatasIn,
		FNamedDataMap& DatasOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override;

protected:

	UE_API EGeometryFlowResult EvaluateInternal(const FDynamicMesh3& Mesh,
										 const FIndexSets& IndexData,
										 const FGenerateSimpleCollisionSettings& Settings,
										 TUniquePtr<FEvaluationInfo>& EvaluationInfo,
										 FCollisionGeometry& OutCollisionGeometry);

};

}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
