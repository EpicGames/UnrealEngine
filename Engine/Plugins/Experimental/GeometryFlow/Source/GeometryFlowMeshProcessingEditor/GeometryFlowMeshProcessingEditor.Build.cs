// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryFlowMeshProcessingEditor : ModuleRules
{
	public GeometryFlowMeshProcessingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "GeoFlowMeshProcEditor";
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"GeometryCore",
				"GeometryAlgorithms",
				"DynamicMesh",
				"GeometryFlowCore",
				"GeometryFlowMeshProcessing"
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"ModelingOperatorsEditorOnly"
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
