// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFlowMeshProcessingEditorModule.h"
#include "Modules/ModuleManager.h"
#include "MeshProcessingEditorNodeRegistration.h"

#define LOCTEXT_NAMESPACE "FGeometryFlowMeshProcessingEditorModule"

void FGeometryFlowMeshProcessingEditorModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	// Register the geometry nodes in this module with the node factory
	UE::GeometryFlow::FMeshProcessingEditorNodeRegistration::RegisterNodes();
}

void FGeometryFlowMeshProcessingEditorModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryFlowMeshProcessingEditorModule, GeometryFlowMeshProcessingEditor)
