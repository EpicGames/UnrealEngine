// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingEditorNodeRegistration.h"
#include "GeometryFlowNodeFactory.h"

#include "MeshProcessingNodes/MeshProcessingDataTypesEditor.h"
#include "MeshProcessingNodes/MeshAutoGenerateUVsNode.h"

void UE::GeometryFlow::FMeshProcessingEditorNodeRegistration::RegisterNodes()
{
	FString CategoryName("UVs");
	GEOMETRYFLOW_REGISTER_NODE_AND_SETTINGS_NODE(MeshAutoGenerateUVs)
}
