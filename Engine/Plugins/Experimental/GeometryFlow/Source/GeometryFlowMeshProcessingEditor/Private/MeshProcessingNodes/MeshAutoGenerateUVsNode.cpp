// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProcessingNodes/MeshAutoGenerateUVsNode.h"

#include "ParameterizationOps/ParameterizeMeshOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshAutoGenerateUVsNode)


using namespace UE::GeometryFlow;


void FMeshAutoGenerateUVsNode::GenerateUVs(const FDynamicMesh3& MeshIn, 
	const FMeshAutoGenerateUVsSettings& Settings, 
	FDynamicMesh3& MeshOut,
	TUniquePtr<FEvaluationInfo>& EvaluationInfo)
{
	// this is horrible - have to copy input mesh so that we can pass a TSharedPtr
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(MeshIn);

	FParameterizeMeshOp ParameterizeMeshOp;
	ParameterizeMeshOp.InputMesh = InputMesh;
	ParameterizeMeshOp.Stretch = Settings.UVAtlasStretch;
	ParameterizeMeshOp.NumCharts = Settings.UVAtlasNumCharts;
	ParameterizeMeshOp.XAtlasMaxIterations = Settings.XAtlasMaxIterations;

	ParameterizeMeshOp.InitialPatchCount = Settings.NumInitialPatches;
	ParameterizeMeshOp.PatchCurvatureAlignmentWeight = Settings.CurvatureAlignment;
	ParameterizeMeshOp.PatchMergingMetricThresh = Settings.MergingThreshold;
	ParameterizeMeshOp.PatchMergingAngleThresh = Settings.MaxAngleDeviationDeg;
	ParameterizeMeshOp.ExpMapNormalSmoothingSteps = Settings.SmoothingSteps;
	ParameterizeMeshOp.ExpMapNormalSmoothingAlpha = Settings.SmoothingAlpha;

	ParameterizeMeshOp.bEnablePacking = Settings.bAutoPack;
	ParameterizeMeshOp.Width = ParameterizeMeshOp.Height = Settings.PackingTargetWidth;
	
	if (FromUEnum(Settings.Method) == EAutoUVMethod::UVAtlas)
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::UVAtlas;
	}
	else if (FromUEnum(Settings.Method) == EAutoUVMethod::XAtlas)
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::XAtlas;
	}
	else
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::PatchBuilder;
	}

	ParameterizeMeshOp.CalculateResult(EvaluationInfo->Progress);

	if (EvaluationInfo && EvaluationInfo->Progress && EvaluationInfo->Progress->Cancelled())
	{
		return;
	}

	TUniquePtr<FDynamicMesh3> ResultMesh = ParameterizeMeshOp.ExtractResult();
	MeshOut = MoveTemp(*ResultMesh);
}
