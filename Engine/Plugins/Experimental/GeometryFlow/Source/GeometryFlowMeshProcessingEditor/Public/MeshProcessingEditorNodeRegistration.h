// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE
{
	namespace GeometryFlow
	{

		struct FMeshProcessingEditorNodeRegistration
		{
			// Registration is required for serialization.
			GEOMETRYFLOWMESHPROCESSINGEDITOR_API static void  RegisterNodes();
		};
	}
}