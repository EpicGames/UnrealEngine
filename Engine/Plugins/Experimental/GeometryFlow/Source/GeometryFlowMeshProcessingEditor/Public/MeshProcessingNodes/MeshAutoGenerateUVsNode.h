// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowTSourceNode.h"
#include "MeshProcessingNodes/MeshProcessingBaseNodes.h"
#include "MeshProcessingNodes/MeshProcessingDataTypesEditor.h"
#include "Serialization/Archive.h"


#include "MeshAutoGenerateUVsNode.generated.h"

#define UE_API GEOMETRYFLOWMESHPROCESSINGEDITOR_API

UENUM()
enum class EGeometryFlow_AutoUVMethod : uint8
{
	PatchBuilder = 0,
	UVAtlas = 1,
	XAtlas = 2
};


USTRUCT()
struct FMeshAutoGenerateUVsSettings
{
	GENERATED_USTRUCT_BODY()

	DECLARE_GEOMETRYFLOW_DATA_TYPE_IDENTIFIER(UE::GeometryFlow::EMeshProcessingDataTypesEditor::MeshAutoGenerateUVsSettings);

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	EGeometryFlow_AutoUVMethod Method = EGeometryFlow_AutoUVMethod::PatchBuilder;

	// UVAtlas parameters
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double UVAtlasStretch = 0.5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int UVAtlasNumCharts = 0;

	// XAtlas parameters
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int XAtlasMaxIterations = 1;

	// PatchBuilder parameters
	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int NumInitialPatches = 100;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double CurvatureAlignment = 1.0;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double MergingThreshold = 1.5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double MaxAngleDeviationDeg = 45.0;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int SmoothingSteps = 5;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	double SmoothingAlpha = 0.25;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	bool bAutoPack = false;

	UPROPERTY(EditAnywhere, Category = "Geometry Flow")
	int PackingTargetWidth = 512;
};


namespace UE
{
namespace GeometryFlow
{
// bring into the namespace

typedef FMeshAutoGenerateUVsSettings FMeshAutoGenerateUVsSettings;

enum class EAutoUVMethod : uint8
{
	PatchBuilder = 0,
	UVAtlas = 1,
	XAtlas = 2
};
static EAutoUVMethod FromUEnum(const  EGeometryFlow_AutoUVMethod& UVsUnwrapType)
{
	return static_cast<EAutoUVMethod>(UVsUnwrapType);
}
static EGeometryFlow_AutoUVMethod ToUEnum(const EAutoUVMethod& UVsUnwrapType)
{
	return static_cast<EGeometryFlow_AutoUVMethod>(UVsUnwrapType);
}


GEOMETRYFLOW_DECLARE_USTRUCT_SETTINGS_TYPES(FMeshAutoGenerateUVsSettings, MeshAutoGenerateUVs, 1);



class FMeshAutoGenerateUVsNode : public TProcessMeshWithSettingsBaseNode<FMeshAutoGenerateUVsSettings>
{
	static constexpr int Version = 1;
	GEOMETRYFLOW_NODE_INTERNAL(FMeshAutoGenerateUVsNode, Version, FNode)
public:
	FMeshAutoGenerateUVsNode() : TProcessMeshWithSettingsBaseNode<FMeshAutoGenerateUVsSettings>()
	{
	}

	virtual void ProcessMesh(
		const FNamedDataMap& DatasIn,
		const FMeshAutoGenerateUVsSettings& Settings,
		const FDynamicMesh3& MeshIn,
		FDynamicMesh3& MeshOut,
		TUniquePtr<FEvaluationInfo>& EvaluationInfo) override
	{
		GenerateUVs(MeshIn, Settings, MeshOut, EvaluationInfo);
	}

	UE_API virtual void GenerateUVs(const FDynamicMesh3& MeshIn, 
		const FMeshAutoGenerateUVsSettings& Settings, 
		FDynamicMesh3& MeshOut, 
		TUniquePtr<FEvaluationInfo>& EvaluationInfo);

};





}	// end namespace GeometryFlow
}	// end namespace UE

#undef UE_API
