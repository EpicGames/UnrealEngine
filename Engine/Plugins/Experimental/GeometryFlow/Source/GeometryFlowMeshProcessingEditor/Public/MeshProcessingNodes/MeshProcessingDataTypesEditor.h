// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowTypes.h"

namespace UE
{
namespace GeometryFlow
{

enum class EMeshProcessingDataTypesEditor
{
	MeshAutoGenerateUVsSettings = (int)EDataTypes::BaseMeshProcessingTypesEditor + 1
};



}	// end namespace GeometryFlow
}	// en
